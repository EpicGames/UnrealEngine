// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuilderKey.h"

namespace UE::DisplayBuilders::BuilderKey
{
	const TCHAR Separator = ',';
	
	namespace KeyNames
	{
		const FName PlaceActors = "PlaceActors";
		const FName None = "None";
	}

}

FName UE::DisplayBuilders::FBuilderKey::ToName() const
{
	return Key;
}

UE::DisplayBuilders::FBuilderKey::FBuilderKey():
	Key( NAME_None )
{
}

FString UE::DisplayBuilders::FBuilderKey::GetKeyWithSuffix(const FName Suffix) const
{
	return Key.ToString()  +  UE::DisplayBuilders::BuilderKey::Separator  + Suffix.ToString();
}

bool UE::DisplayBuilders::FBuilderKey::IsNone() const
{
	return Key.IsNone();
}

const UE::DisplayBuilders::FBuilderKeys& UE::DisplayBuilders::FBuilderKeys::Get()
{
	static const FBuilderKeys Keys;
	return Keys;
}

const UE::DisplayBuilders::FBuilderKey& UE::DisplayBuilders::FBuilderKeys::PlaceActors() const 
{
	static const  UE::DisplayBuilders::FBuilderKey Key{ BuilderKey::KeyNames::PlaceActors };
	return Key;
}

const UE::DisplayBuilders::FBuilderKey& UE::DisplayBuilders::FBuilderKeys::None() const 
{
	static const  UE::DisplayBuilders::FBuilderKey Key{ BuilderKey::KeyNames::None };
	return Key;
}

UE::DisplayBuilders::FBuilderKeys::FBuilderKeys()
{
}
