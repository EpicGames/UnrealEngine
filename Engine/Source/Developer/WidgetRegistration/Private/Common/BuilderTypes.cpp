// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common/BuilderTypes.h"

UE::DisplayBuilders::FLabelAndIconArgs::FLabelAndIconArgs(FText InLabel, FSlateIcon InIcon):
	Label( InLabel )
	, Icon( InIcon )
{
}
