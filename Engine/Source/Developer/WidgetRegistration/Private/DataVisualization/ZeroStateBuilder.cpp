// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataVisualization/ZeroStateBuilder.h"
#include "Styling/AppStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"

FZeroStateBuilder::FZeroStateBuilder( UE::DisplayBuilders::FLabelAndIconArgs LabelAndIconArgs ) :
Icon( LabelAndIconArgs.Icon )
, Label( LabelAndIconArgs.Label )
{
}

TSharedPtr<SWidget> FZeroStateBuilder::GenerateWidget()
{
	// TODO: extract style info into class
	return SNew( SVerticalBox )
			+ SVerticalBox::Slot()
			.HAlign( HAlign_Center )
			.AutoHeight()
			[
				SNew(SBox)
				.MaxDesiredWidth(40.f)
				.Padding(0.f, 20.f )
				[
					SNew(SImage )
					.Image( Icon.GetIcon() )
					.DesiredSizeOverride( FVector2D(40.f, 40.f ) )
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBox)
				.MaxDesiredWidth( 250.f )
				[
					SNew(STextBlock)
					.Text( Label )
					.TextStyle( FAppStyle::Get(), "HintText")
					.OverflowPolicy( ETextOverflowPolicy::MultilineEllipsis )
					.Justification( ETextJustify::Center )
					.AutoWrapText( true )
				]
			];
}

void FZeroStateBuilder::UpdateWidget()
{
}

void FZeroStateBuilder::ResetWidget()
{
}
