// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Common/BuilderTypes.h"
#include "Layout/Containers/SlateBuilder.h"
#include "Styles/BuilderIconKeys.h"

#define LOCTEXT_NAMESPACE "ZeroStateBuilder"

/**
 * A builder which builds a view of a "Zero State" which can be displayed when no objects are available for that view. It has an icon which
 * shows some indication of the missing data, and some text which explains the state.
 */
class FZeroStateBuilder : public FSlateBuilder
{
public:
	/**
	 * The constructor which takes a FLabelAndIconArgs to initialize the Zero state
	 * 
	 * @param LabelAndIconArgs a parameter object to provide the label and icon information 
	 */
	explicit FZeroStateBuilder( UE::DisplayBuilders::FLabelAndIconArgs LabelAndIconArgs =
	UE::DisplayBuilders::FLabelAndIconArgs{ LOCTEXT("ZeroStateBuilderDefaultLabel", "No items available." )
	, FBuilderIconKeys::Get().ZeroStateDefaultMedium().GetSlateIcon() } );

	/**
	 * Build the Zero state SWidget and return it
	 */
	virtual TSharedPtr<SWidget> GenerateWidget() override;

private:
	/**
	 * Keeping UpdateWidget private until/unless there is a need for it
	 */
	virtual void UpdateWidget() override;

	/**
	 * Keeping ResetWidget private until/unless there is a need for it
	 */
	virtual void ResetWidget() override;

	FSlateIcon Icon;
	FText Label;
};

#undef LOCTEXT_NAMESPACE