// Copyright Epic Games, Inc. All Rights Reserved.


#include "DataVisualization/ZeroStateBuilderTemplates.h"

#define LOCTEXT_NAMESPACE "ZeroStateBuilderTemplates"

const FZeroStateBuilderTemplates& FZeroStateBuilderTemplates::Get()
{
	static const FZeroStateBuilderTemplates Templates;
	return Templates;
}

TSharedRef<FZeroStateBuilder> FZeroStateBuilderTemplates::GetDefault( FText DefaultText ) const
{
	const FText LabelText = DefaultText.IsEmpty() ?
		LOCTEXT("ZeroStateDefaultLabel", "No results.") : DefaultText;
	const FSlateIcon SlateIcon = FBuilderIconKeys::Get().ZeroStateDefaultMedium().GetSlateIcon();
	
	return MakeShared<FZeroStateBuilder>(
		UE::DisplayBuilders::FLabelAndIconArgs
		{
			LabelText,
			SlateIcon
		}
	);
}

TSharedRef<FZeroStateBuilder> FZeroStateBuilderTemplates::GetFavorites( FText NoFavoritesAvailableText ) const
{
	const FText LabelText = NoFavoritesAvailableText.IsEmpty() ?
		LOCTEXT("ZeroStateFavoritesLabel", "No favorites." ) :
		NoFavoritesAvailableText;

	const FSlateIcon SlateIcon = FBuilderIconKeys::Get().ZeroStateFavoritesMedium().GetSlateIcon();
	
	return MakeShared<FZeroStateBuilder>(
			UE::DisplayBuilders::FLabelAndIconArgs
			{
				LabelText,
				SlateIcon
			}
		);
}

#undef LOCTEXT_NAMESPACE