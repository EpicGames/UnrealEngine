// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ZeroStateBuilder.h"

/**
 * FZeroStateBuilderTemplates provides some templates for Zero State Builders.
 */
class FZeroStateBuilderTemplates
{
public:
	static const FZeroStateBuilderTemplates& Get();

	/**
	 * A default Zero state builder which gives a general default icon and text for unavailable items
	 *
	 * @param DefaultText the text shown below the icon in the zero state view
	 */
	TSharedRef<FZeroStateBuilder> GetDefault( FText DefaultText = FText::GetEmpty() ) const;

	/**
	 * A Zero state builder which gives an icon and text for no favorites items available
	 
	 * @param NoFavoritesAvailableText the text shown below the icon in the zero state view
	 */
	TSharedRef<FZeroStateBuilder> GetFavorites( FText NoFavoritesAvailableText = FText::GetEmpty() ) const;
};

