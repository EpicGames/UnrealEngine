// Copyright Epic Games, Inc. All Rights Reserved.

#include "FToolkitWidgetStyle.h"
#include UE_INLINE_GENERATED_CPP_BY_NAME(FToolkitWidgetStyle)

FToolkitWidgetStyle& FToolkitWidgetStyle::SetActiveToolTitleBorderPadding(const FMargin& InActiveToolTitleBorderPadding)
{
	ActiveToolTitleBorderPadding = InActiveToolTitleBorderPadding; return *this;
}

FToolkitWidgetStyle& FToolkitWidgetStyle::SetToolContextTextBlockPadding(const FMargin& InToolContextTextBlockPadding)
{
	ToolContextTextBlockPadding = InToolContextTextBlockPadding; return *this;
}

FToolkitWidgetStyle& FToolkitWidgetStyle::SetTitleBackgroundBrush(const FSlateBrush& InPaletteTitleBackgroundBrush)
{
	TitleBackgroundBrush = InPaletteTitleBackgroundBrush; return *this;
}

FToolkitWidgetStyle& FToolkitWidgetStyle::SetToolDetailsBackgroundBrush(const FSlateBrush& InToolDetailsBackgroundBrush)
{
	ToolDetailsBackgroundBrush = InToolDetailsBackgroundBrush; return *this;
}

const FToolkitWidgetStyle& FToolkitWidgetStyle::GetDefault()
{
	static FToolkitWidgetStyle Default;
	return Default;
}

const FName FToolkitWidgetStyle::TypeName(TEXT("FToolkitWidgetStyle"));

void FToolkitWidgetStyle::GetResources(TArray<const FSlateBrush*>& OutBrushes) const
{
	OutBrushes.Add(&TitleBackgroundBrush);
}

FToolkitWidgetStyle& FToolkitWidgetStyle::SetTitleForegroundColor(const FSlateColor& InTitleForegroundColor)
{
	TitleForegroundColor = InTitleForegroundColor; return *this;
}

FToolkitWidgetStyle& FToolkitWidgetStyle::SetTitlePadding(const FMargin& InTitlePadding)
{
	TitlePadding = InTitlePadding; return *this;
}

FToolkitWidgetStyle& FToolkitWidgetStyle::SetTitleFont(const FSlateFontInfo& InTitleFont)
{
	TitleFont = InTitleFont; return *this;
}

