// Copyright Epic Games, Inc. All Rights Reserved.

#include "Inputs/BuilderCommandCreationManager.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Inputs/BuilderInput.h"

#define LOCTEXT_NAMESPACE "BuilderCommandCreationManager"

FBuilderCommandCreationManager::FBuilderCommandCreationManager()
	: TCommands<FBuilderCommandCreationManager>(
		TEXT("BuilderCommandCreationManager"), 
		LOCTEXT("BuilderCommandCreationManager", "Dynamic FUI Commands"), 
		NAME_None, 
		"FBuilderCommandCreationManagerStyle")
{
}

void FBuilderCommandCreationManager::RegisterCommands()
{
	/*
	 * we must put in one command for the singleton to stay valid after it is registered. Since this is for dynamic commands we register a default
	 */
	static const FName Default{ "Default" };
	static UE::DisplayBuilders::FBuilderInput Input{}; 
	Input.Name = Default;
	Input.Label = FText::FromName(Default);
	RegisterCommandForBuilder( Input );
}

void FBuilderCommandCreationManager::RegisterCommandForBuilder( UE::DisplayBuilders::FBuilderInput& Input ) const
{
	if (IsRegistered() && !Input.IsNameNone() )
	{
		const TSharedPtr<FBuilderCommandCreationManager> Commands = GetInstance().Pin();

		TSharedPtr<FUICommandInfo> NewCommandInfo;

		FUICommandInfo::MakeCommandInfo(
			Commands->AsShared(),
			NewCommandInfo,
			Input.Name,
			Input.Label,
			Input.Tooltip,
			Input.Icon,
			EUserInterfaceActionType::RadioButton,
			Input.DefaultChords );

		Input.UICommandInfo = NewCommandInfo;
		Input.ButtonArgs.Command = NewCommandInfo;
	};
}

void FBuilderCommandCreationManager::UnregisterCommandForBuilder( UE::DisplayBuilders::FBuilderInput& Input ) const
{
	if ( Input.UICommandInfo.IsValid() )
	{
		FUICommandInfo::UnregisterCommandInfo( GetInstance().Pin().ToSharedRef(), Input.UICommandInfo.ToSharedRef() );		
	}
}

#undef LOCTEXT_NAMESPACE
