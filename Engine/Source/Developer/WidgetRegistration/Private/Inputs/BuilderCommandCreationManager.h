// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Inputs/BuilderInput.h"

namespace UE::DisplayBuilders
{
	class FBuilderInput;
}

/**
* FBuilderCommandCreationManager creates dynamic FUICommandInfos that can be used with toolbars, menus, and other command-centric objects and
* fills out the associated information in FBuilderInputs
*/
class FBuilderCommandCreationManager : public TCommands<FBuilderCommandCreationManager>
{
	
public:
	/** default constructor  */
	FBuilderCommandCreationManager();

	/**
	 * Creates the current commands specified in FUICommandInfoArray
	 */
	virtual void RegisterCommands() override;

	/**
   	* Given a FBuilderInput, registers a TSharedPtr<FUICommandInfo> which can be used to create Slate 
   	* toolbars, menus, and other command-centric objects and initializes the Command information for the FBuilderInput 
   	*
   	* @param OutBuilderInput the FBuilderInput for which to register and initialize the related TSharedPtr<FUICommandInfo>
   	*/
	void RegisterCommandForBuilder( UE::DisplayBuilders::FBuilderInput& OutBuilderInput ) const;

	/**
    * Given a FBuilderInput, unregisters and tears down the related TSharedPtr<FUICommandInfo> 
    *
    * @param OutBuilderInput the FBuilderInput for which to unregister and tear down TSharedPtr<FUICommandInfo>
    */
	void UnregisterCommandForBuilder( UE::DisplayBuilders::FBuilderInput& OutBuilderInput ) const;
};