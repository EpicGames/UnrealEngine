// Copyright Epic Games, Inc. All Rights Reserved.

#include "Inputs/BuilderInputManager.h"
#include "BuilderCommandCreationManager.h"
#include "Framework/Commands/UICommandInfo.h"

FBuilderInputManager& FBuilderInputManager::Get()
{
	static FBuilderInputManager Manager;
	return Manager;
}

void FBuilderInputManager::Initialize()
{
	FBuilderCommandCreationManager::Register();
}

void FBuilderInputManager::Shutdown()
{
	FBuilderCommandCreationManager::Unregister();
}

const FBuilderCommandCreationManager& FBuilderInputManager::GetCommandManager()
{
	static const FBuilderCommandCreationManager& CommandManager = FBuilderCommandCreationManager::Get();
	return CommandManager;
}
