// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/UICommandInfo.h"

class FBuilderCommandCreationManager;

namespace UE::DisplayBuilders
{
	class FBuilderInput;
}

/**
* BuilderInputManager provides access to utilities and information regarding user input.
*/
class FBuilderInputManager
{
public:

	/**
	* Gets the singleton instance of FBuilderInputManager
	*/
	static FBuilderInputManager& Get();

	/**
	 * Registers and initializes the Command binding context.
	 */
	static void Initialize();

	/**
	 * Unregisters the Command context.
	 */
	static void Shutdown();

	/**
	 * @return the Command manager for Builders
	 */
	const FBuilderCommandCreationManager& GetCommandManager();
};
