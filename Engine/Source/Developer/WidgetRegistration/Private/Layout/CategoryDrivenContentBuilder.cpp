// Copyright Epic Games, Inc. All Rights Reserved.


#include "Layout/CategoryDrivenContentBuilder.h"

#include "Containers/ColumnWrappingContainerTemplates.h"
#include "DataVisualization/ZeroStateBuilder.h"
#include "DataVisualization/ZeroStateBuilderTemplates.h"
#include "Templates/SharedPointer.h"
#include "Framework/MultiBox/SToolBarButtonBlock.h"
#include "Inputs/BuilderInputManager.h"
#include "Input/DragAndDrop.h"
#include "Layout/Containers/ColumnWrappingContainer.h"
#include "Layout/Containers/SimpleTitleContainer.h"
#include "Persistence/BuilderPersistenceManager.h"
#include "Layout/Containers/SlateBuilder.h"
#include "Styling/ToolBarStyle.h"

#define LOCTEXT_NAMESPACE "CategoryDrivenContentBuilder"

const TArray<FName>& FCategoryDrivenContentBuilder::GetFavorites() const
{
	return Favorites;
}

TSharedRef<SWidget> FCategoryDrivenContentBuilder::CreateFavoritesContextMenu( FString FavoritesItemName )
{
	bool bInShouldCloseWindowAfterMenuSelection = true;
	TSharedPtr< const FUICommandList > InCommandList;
	
	FMenuBuilder MenuBuilder(bInShouldCloseWindowAfterMenuSelection, InCommandList);

	const FText ItemText = Favorites.Contains( FavoritesItemName ) ?
		LOCTEXT("CategoryDrivenContentBuilder_RemoveFromFavorites", "Remove from Favorites") :
		LOCTEXT("CategoryDrivenContentBuilder_AddToFavorites", "Add to Favorites");
		
	const FUIAction ItemAction(FExecuteAction::CreateSP( this, &FCategoryDrivenContentBuilder::ToggleFavorite, FName( FavoritesItemName ) ) );
	MenuBuilder.AddMenuEntry(ItemText, ItemText, FSlateIcon(), ItemAction);
		
	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> FCategoryDrivenContentBuilder::CreateShowCategoryLabelsContextMenu()
{
	const bool bInShouldCloseWindowAfterMenuSelection = true;
	const TSharedPtr< const FUICommandList > CommandList = nullptr;
	
	FMenuBuilder MenuBuilder(bInShouldCloseWindowAfterMenuSelection, CommandList);

	const FText ItemText = CategoryButtonLabelVisibility.IsVisible() ?
		LOCTEXT("CategoryDrivenContentBuilder_HideLabels", "Hide Labels") :
		LOCTEXT("CategoryDrivenContentBuilder_ShowLabels", "Show Labels");
		
	const FUIAction ItemAction(FExecuteAction::CreateSP(this, &FCategoryDrivenContentBuilder::ToggleShowLabels));
	MenuBuilder.AddMenuEntry(ItemText, ItemText, FSlateIcon(), ItemAction);
		
	return MenuBuilder.MakeWidget();
}

FCategoryDrivenContentBuilder::FCategoryDrivenContentBuilder(FCategoryDrivenContentBuilderArgs& Args):
	FCategoryDrivenContentBuilderBase(Args )
	, FavoritesCategoryName( Args.FavoritesCommandName )
	, CategoryLabel( Args.CategoryLabel )
	, TitleContainer( nullptr )
	, bIsFilledWithWidget( false )
	, bShowNoCategorySelection( false )
{
	Favorites = UBuilderPersistenceManager::Get()->GetFavoritesNames( BuilderKey );
	const bool bCategoryButtonLabelVisibilityBool = UBuilderPersistenceManager::Get()->GetShowButtonLabels( BuilderKey, CategoryButtonLabelVisibility.IsVisible() );
	CategoryButtonLabelVisibility = bCategoryButtonLabelVisibilityBool ? EVisibility::Visible : EVisibility::Collapsed;
	ActiveCategoryName = Args.ActiveCategoryName;
}

FCategoryDrivenContentBuilder::~FCategoryDrivenContentBuilder()
{
	UpdateContentForCategoryDelegate.Unbind();
}

void FCategoryDrivenContentBuilder::UpdateContentForCategory( FName InActiveCategoryName, FText InActiveCategoryText )
{
	bShowNoCategorySelection = false;
	ChildBuilderArray.Empty();

	ActiveCategoryName = InActiveCategoryName;
	
	UpdateWidget();
}

void FCategoryDrivenContentBuilder::ToggleFavorite( FName InFavoriteCommandName )
{
	if  ( Favorites.Contains( InFavoriteCommandName  ) )
	{
		Favorites.Remove( InFavoriteCommandName );
	}
	else
	{
		Favorites.Add( InFavoriteCommandName );
	}

	UBuilderPersistenceManager::Get()->PersistFavoritesNames( BuilderKey, Favorites );
	UpdateWidget();
}

void FCategoryDrivenContentBuilder::ToggleShowLabels()
{
	CategoryButtonLabelVisibility = CategoryButtonLabelVisibility.IsVisible() ? EVisibility::Collapsed : EVisibility::Visible;
	UBuilderPersistenceManager::Get()->PersistShowButtonLabels( BuilderKey, CategoryButtonLabelVisibility.IsVisible() );

	const bool bShowReinitialize = true;
	RefreshCategoryToolbarWidget( bShowReinitialize );
	InitializeCategoryButtons();
	CreateWidget();
}

void FCategoryDrivenContentBuilder::AddFavorite(FName InFavoriteCommandName)
{
	if ( !Favorites.Contains( InFavoriteCommandName ) )
	{
		ToggleFavorite( InFavoriteCommandName );
	}
}

void FCategoryDrivenContentBuilder::AddBuilder(TSharedRef<SWidget> Widget)
{
	bIsFilledWithWidget = false;
	ChildBuilderArray.Add( MakeShared<FSlateBuilder>( Widget ) );
}

void FCategoryDrivenContentBuilder::FillWithBuilder(TSharedRef<SWidget> Widget)
{
	ChildBuilderArray.Empty();
	ChildBuilderArray.Add( MakeShared<FSlateBuilder>( Widget ) );
	bIsFilledWithWidget = true;
}

void FCategoryDrivenContentBuilder::ClearCategoryContent()
{
	MainContentVerticalBox->ClearChildren();
	ChildBuilderArray.Empty();
}

void FCategoryDrivenContentBuilder::InitializeCategoryToolbar()
{
	if (!LoadPaletteToolBarBuilder)
	{
		return;
	}

	LoadPaletteToolBarBuilder->SetLabelVisibility(CategoryButtonLabelVisibility);
	
	for ( TTuple<FName, UE::DisplayBuilders::FBuilderInput>  Pair : CategoryNameToBuilderInputMap )
	{
		FButtonArgs ButtonArgs = Pair.Value.ButtonArgs;
		const TSharedPtr<const FUICommandInfo> Command = ButtonArgs.Command;
		const FName CommandName = Command->GetCommandName();
		ButtonArgs.OnGetMenuContent.BindSP(this, &FCategoryDrivenContentBuilder::CreateShowCategoryLabelsContextMenu );
		
		LoadToolPaletteCommandList->MapAction(
			Command,
			FExecuteAction::CreateSP(this, &FCategoryDrivenContentBuilder::UpdateContentForCategory,
				CommandName, Command->GetLabel() ),
			FCanExecuteAction::CreateLambda([] { return true; }),
			FGetActionCheckState::CreateLambda([this, CommandName] ()
			{
				if ( !bShowNoCategorySelection )
				{
					return ActiveCategoryName == CommandName ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;		
				}
				return ECheckBoxState::Unchecked;
			}
			)
		);

		LoadPaletteToolBarBuilder->AddToolBarButton( ButtonArgs );
	}
}

TSharedPtr<SWidget> FCategoryDrivenContentBuilder::GenerateWidget()
{
	TSharedRef<SWidget> Widget = FCategoryDrivenContentBuilderBase::GenerateWidget().ToSharedRef();

	if ( !ActiveCategoryName.IsNone() )
	{
		UpdateContentForCategory( ActiveCategoryName );
	}

	return Widget;
}

void FCategoryDrivenContentBuilder::UpdateWidget()
{
	if ( !CategoryNameToBuilderInputMap.IsEmpty() )
	{
		CategoryLabel = FText::GetEmpty();
	
		if ( const UE::DisplayBuilders::FBuilderInput* Input = CategoryNameToBuilderInputMap.Find( ActiveCategoryName ) )
		{
			CategoryLabel = (*Input).Label;
		
			if ( LoadPaletteToolBarBuilder && !bShowNoCategorySelection )
			{
				LoadPaletteToolBarBuilder->SetLastSelectedCommandIndex( (*Input).Index );
			}
		}

		FName Name = bShowNoCategorySelection ? NAME_None : ActiveCategoryName;
		FText Text = bShowNoCategorySelection ? FText::GetEmpty() : CategoryLabel;
		UpdateContentForCategoryDelegate.ExecuteIfBound(  Name, Text );
		FSimpleTitleContainerArgs Args{ CategoryLabel };
		Args.bIsHeaderHiddenOnCreate = bShowNoCategorySelection;
		TSharedPtr<FZeroStateBuilder> ZeroStateBuilder;
	
		TitleContainer = MakeShared<FSimpleTitleContainer>( Args );
		bool bIsFavoritesCategory = ActiveCategoryName == FavoritesCategoryName;
	
		if ( bIsFilledWithWidget && ChildBuilderArray.Num() == 1 )
		{
			TitleContainer->SetBody( ChildBuilderArray[0] );
		}
		else if ( !ChildBuilderArray.IsEmpty() )
		{
			if ( !ColumnWrappingContainer.IsValid() )
			{
				ColumnWrappingContainer = FColumnWrappingContainerTemplates::Get().GetBestFitColumnsWithSmallCells();			
			}

			ColumnWrappingContainer->SetBuilders( ChildBuilderArray );
			TitleContainer->SetBody
			( 
				ColumnWrappingContainer->GenerateWidgetSharedRef()
			);
		}
		else
		{
			ZeroStateBuilder = bIsFavoritesCategory ?
				                   FZeroStateBuilderTemplates::Get().GetFavorites(
					                   LOCTEXT("CategoryDrivenContentBuilder_NoFavoritesYet",
					                           "No favorites yet.\n\n To create favorites, right-click on items from other categories and add them to the Favorites.")) :

				                   FZeroStateBuilderTemplates::Get().GetDefault(
					                   LOCTEXT("CategoryDrivenContentBuilder_NoActorsMatchSearch",
					                           "No actors match your search."));
		}

		const bool bIsZeroState = ZeroStateBuilder.IsValid();

		MainContentVerticalBox->ClearChildren();
		MainContentVerticalBox->AddSlot()
		                      .FillHeight(1.0f)
		                      .VAlign(VAlign_Fill)
		                      .HAlign( bIsZeroState ? HAlign_Center : HAlign_Fill )
		[
			bIsZeroState ?
				SNew(SBox)
				[
					ZeroStateBuilder->GenerateWidgetSharedRef()
				]:
				TitleContainer->GenerateWidgetSharedRef()
		];
	}
}

void FCategoryDrivenContentBuilder::SetShowNoCategorySelection(bool bInShowNoCategorySelection)
{
	if (bShowNoCategorySelection != bInShowNoCategorySelection)
	{
		bShowNoCategorySelection = bInShowNoCategorySelection;
		const UE::DisplayBuilders::FBuilderInput* Input = CategoryNameToBuilderInputMap.Find( ActiveCategoryName );

		if ( bShowNoCategorySelection )
		{
			LoadPaletteToolBarBuilder->SetLastSelectedCommandIndex( INDEX_NONE );
		}
		else if ( Input != nullptr)
		{
			LoadPaletteToolBarBuilder->SetLastSelectedCommandIndex( (*Input).Index );
		}
	}
}

void FCategoryDrivenContentBuilder::InitializeCategoryButtons(TArray<UE::DisplayBuilders::FBuilderInput> InBuilderInputArray)
{
	BuilderInputArray = InBuilderInputArray;
	InitializeCategoryButtons();
}

void FCategoryDrivenContentBuilder::InitializeCategoryButtons()
{
	CategoryNameToBuilderInputMap.Empty();
	
	for ( int32 Index = 0; Index < BuilderInputArray.Num(); Index++ )
	{
		UE::DisplayBuilders::FBuilderInput& BuilderInput = BuilderInputArray[Index];
		BuilderInput.Index = Index;
		
		if ( BuilderInput.Name == FavoritesCategoryName && GetDecoratedButtonDelegate.IsBound() )
		{
			BuilderInput.ButtonArgs.GetDecoratedButtonDelegate = GetDecoratedButtonDelegate;
			BuilderInput.ButtonArgs.IconOverride = FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Favorites.Small");
		}
		CategoryNameToBuilderInputMap.Add( BuilderInput.Name, BuilderInput );
	}
	
	constexpr bool bForceSmallIcons = true; 
	
	LoadToolPaletteCommandList = MakeShared<FUICommandList>();
	LoadPaletteToolBarBuilder = MakeShared<FVerticalToolBarBuilder>(LoadToolPaletteCommandList, FMultiBoxCustomization::None, TSharedPtr<FExtender>(), bForceSmallIcons);
	
	const FName StyleName = GetCategoryToolBarStyleName(); 
	LoadPaletteToolBarBuilder->SetStyle(&FAppStyle::Get(), StyleName );
	LoadPaletteToolBarBuilder->SetLabelVisibility( CategoryButtonLabelVisibility );

	InitializeCategoryToolbar();
}

#undef LOCTEXT_NAMESPACE
