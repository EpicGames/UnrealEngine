// Copyright Epic Games, Inc. All Rights Reserved.

#include "Layout/CategoryDrivenContentBuilderBase.h"
#include "ToolbarRegistrationArgs.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SBoxPanel.h"
#include "ToolElementRegistry.h"
#include "Framework/Commands/UICommandList.h"
#include "Layout/SeparatorBuilder.h"
#include "Layout/SeparatorTemplates.h"
#include "Layout/Visibility.h"
#include "Widgets/SInvalidationPanel.h"

namespace UE::CategoryDrivenContentBuilderBase::Private
{
	const FName StyleName = TEXT("Name");
	const float LabelledButtonToolbarButtonWidth = 64;
	const float UnlabelledButtonToolbarButtonWidth = 44;
	
}

FToolElementRegistry FCategoryDrivenContentBuilderBase::ToolRegistry = FToolElementRegistry::Get();

FCategoryDrivenContentBuilderBase::FCategoryDrivenContentBuilderBase ( FName InBuilderName ) :
	FToolElementRegistrationArgs( InBuilderName )
	, CategoryReclickBehavior( ECategoryReclickBehavior::NoEffect)
	, BuilderName( InBuilderName )
{
}

FCategoryDrivenContentBuilderBase::FCategoryDrivenContentBuilderBase( FCategoryDrivenContentBuilderArgs& Args ) :
	FToolElementRegistrationArgs( Args.Key )
	, GetDecoratedButtonDelegate( Args.GetDecoratedButtonDelegate )
	, CategoryReclickBehavior( Args.CategoryReclickBehavior )
	, BuilderName(Args.BuilderName)
	, ActiveCategoryName( Args.ActiveCategoryName )
{
}

FCategoryDrivenContentBuilderBase::~FCategoryDrivenContentBuilderBase()
{
	if (VerticalToolbarElement.IsValid())
	{
		ToolRegistry.UnregisterElement(VerticalToolbarElement.ToSharedRef());
	}
}

TSharedPtr<FToolBarBuilder> FCategoryDrivenContentBuilderBase::GetLoadPaletteToolbar()
{
	return LoadPaletteToolBarBuilder;
}

void FCategoryDrivenContentBuilderBase::InitCategoryToolbarContainerWidget()
{
	if (!CategoryToolbarVBox.IsValid())
	{
		CategoryToolbarVBox = SNew(SVerticalBox)
			.Visibility(CategoryToolbarVisibility);
	}
	else
	{
		CategoryToolbarVBox->ClearChildren();
		CategoryToolbarVBox->SetVisibility(CategoryToolbarVisibility);
	}

	float ToolbarBoxWidth = UE::CategoryDrivenContentBuilderBase::Private::UnlabelledButtonToolbarButtonWidth;

	if ( CategoryButtonLabelVisibility.IsVisible() )
	{
		ToolbarBoxWidth = UE::CategoryDrivenContentBuilderBase::Private::LabelledButtonToolbarButtonWidth;
	}
	
	CategoryToolbarVBox->AddSlot()
	.Padding(0.f)
	[
		SNew(SBox)
		.WidthOverride( ToolbarBoxWidth )
		[
			CreateToolbarWidget()
		]
	];
}

void FCategoryDrivenContentBuilderBase::RefreshCategoryToolbarWidget(bool bShouldReinitialize)
{
	if ( !LoadPaletteToolBarBuilder.IsValid() )
	{
		return;
	}
	FToolElementRegistrationKey ElementKey = FToolElementRegistrationKey(BuilderName, EToolElement::Toolbar);
	VerticalToolbarElement = ToolRegistry.GetToolElementSP(ElementKey);
	LoadPaletteToolBarBuilder->SetLabelVisibility( CategoryButtonLabelVisibility );
	const TSharedRef<FToolbarRegistrationArgs> VerticalToolbarRegistrationArgs = MakeShareable<FToolbarRegistrationArgs>(
		new FToolbarRegistrationArgs(LoadPaletteToolBarBuilder.ToSharedRef()));
	
	if (VerticalToolbarElement.IsValid() && bShouldReinitialize)
	{
		ToolRegistry.UnregisterElement(VerticalToolbarElement.ToSharedRef());
		VerticalToolbarElement = nullptr;
	}
	
	if (!VerticalToolbarElement.IsValid() || bShouldReinitialize)
	{
		VerticalToolbarElement = MakeShareable(new FToolElement
			(BuilderName,
			VerticalToolbarRegistrationArgs));
		ToolRegistry.RegisterElement(VerticalToolbarElement.ToSharedRef());
	}

	VerticalToolbarElement->SetRegistrationArgs(VerticalToolbarRegistrationArgs);
	InitCategoryToolbarContainerWidget();
}

TSharedPtr<SWidget> FCategoryDrivenContentBuilderBase::GenerateWidget()
{
	if ( !LoadPaletteToolBarBuilder.IsValid() )
	{
		return SNullWidget::NullWidget;
	}
	if (!ToolkitWidgetContainerVBox )
	{
		CreateWidget();
	}
	return ToolkitWidgetContainerVBox.ToSharedRef();
}


void FCategoryDrivenContentBuilderBase::CreateWidget()
{
	MainContentVerticalBox = MainContentVerticalBox.IsValid() ? MainContentVerticalBox : SNew(SVerticalBox);
	MainContentVerticalBox->ClearChildren();
	RefreshCategoryToolbarWidget();
	UpdateContentForCategory( ActiveCategoryName );
	

	ToolkitWidgetContainerVBox = SNew(SVerticalBox)
	+ SVerticalBox::Slot().AutoHeight() [ *FSeparatorTemplates::SmallHorizontalPanelNoBorder()  ]
	+ SVerticalBox::Slot().AutoHeight() [ *FSeparatorTemplates::SmallHorizontalBackgroundNoBorder() ];

	TSharedPtr<SWidget> MainSplitter = 
		SNew(SSplitter)
		.PhysicalSplitterHandleSize(2.0f)
		+ SSplitter::Slot()
		.Resizable(false)
		.SizeRule(SSplitter::SizeToContent)
		[
#if UE_SLATE_WITH_DYNAMIC_INVALIDATION
			SNew(SInvalidationPanel)
			.UseDynamicInvalidation(true)
			[
#endif
				CategoryToolbarVBox.ToSharedRef()
#if UE_SLATE_WITH_DYNAMIC_INVALIDATION
			]
#endif
		]

		+ SSplitter::Slot()
		.SizeRule(SSplitter::FractionOfParent)
		[
#if UE_SLATE_WITH_DYNAMIC_INVALIDATION
			SNew(SInvalidationPanel)
			.UseDynamicInvalidation(true)
			[
#endif
				MainContentVerticalBox->AsShared()
#if UE_SLATE_WITH_DYNAMIC_INVALIDATION
			]
#endif
		];
	
		ToolkitWidgetContainerVBox->AddSlot()
    	.VAlign(VAlign_Fill)
	    .FillHeight(1)
		[
			MainSplitter->AsShared()
		];
}

FCategoryDrivenContentBuilderArgs::FCategoryDrivenContentBuilderArgs(
	FName InBuilderName
	, const UE::DisplayBuilders::FBuilderKey& InKey ) :
		Key( InKey )
		, BuilderName( InBuilderName )
		, bShowCategoryButtonLabels(false)
		, CategoryReclickBehavior(FCategoryDrivenContentBuilderBase::ECategoryReclickBehavior::NoEffect)
		, FavoritesCommandName( NAME_None )
		, CategoryLabel( FText::GetEmpty() )
		, ActiveCategoryName( NAME_None )
{
}

void FCategoryDrivenContentBuilderBase::SetCategoryButtonLabelVisibility(EVisibility Visibility)
{
	CategoryButtonLabelVisibility = Visibility;
	InitializeCategoryToolbar();
}

void FCategoryDrivenContentBuilderBase::SetCategoryButtonLabelVisibility(bool bIsCategoryButtonLabelVisible)
{
	SetCategoryButtonLabelVisibility(bIsCategoryButtonLabelVisible ? EVisibility::Visible : EVisibility::Collapsed);
}


TSharedRef<SWidget> FCategoryDrivenContentBuilderBase::CreateToolbarWidget() const
{
	return ToolRegistry.GenerateWidget(VerticalToolbarElement.ToSharedRef());
}


FName FCategoryDrivenContentBuilderBase::GetCategoryToolBarStyleName() const
{
	return CategoryButtonLabelVisibility.IsVisible() ?
		"CategoryDrivenContentBuilderToolbarWithLabels" :
		"CategoryDrivenContentBuilderToolbarWithoutLabels";
}