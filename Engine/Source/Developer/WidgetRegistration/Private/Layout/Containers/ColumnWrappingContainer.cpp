// Copyright Epic Games, Inc. All Rights Reserved.

#include "Layout/Containers/ColumnWrappingContainer.h"

#include "Widgets/Layout/SBorder.h"
#include "Styles/SlateBrushTemplates.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SUniformWrapPanel.h"


FColumnWrappingContainerArgs::FColumnWrappingContainerArgs( float InCellHeight, int32 InNumColumnsOverride, FName InIdentifier ) :
	FWidgetContainerArgs( InIdentifier )
	, NumColumns( InNumColumnsOverride )
	, CellHeight( InCellHeight )
{
}

FColumnWrappingContainer::FColumnWrappingContainer(FColumnWrappingContainerArgs&& Args):
	FWidgetContainer ( Args )
	, NumColumns( Args.NumColumns )
	, CellHeight( Args.CellHeight )
{
	Initialize();
}

FColumnWrappingContainer::FColumnWrappingContainer(const FColumnWrappingContainerArgs& Args):
	FWidgetContainer ( Args )
	, NumColumns( Args.NumColumns )
	, CellHeight( Args.CellHeight )
{
	Initialize();
}

FColumnWrappingContainer& FColumnWrappingContainer::SetNumColumns(const int32& InNumColumns)
{
	NumColumns = InNumColumns;
	UniformWrapPanel->SetNumColumnsOverride( NumColumns );
	return *this;
}

void FColumnWrappingContainer::Empty()
{
	FWidgetContainer::Empty();
	UniformWrapPanel->ClearChildren();
}

void FColumnWrappingContainer::CreateAndPositionWidgetAtIndex(int32 ChildBuilderIndex)
{
	UniformWrapPanel->AddSlot()
	[
		GetBuilderAtIndex( ChildBuilderIndex )->GenerateWidgetSharedRef()
	];
}

void FColumnWrappingContainer::Initialize()
{
	SAssignNew (MainContentSBorder,  SBorder )
 		.Padding(8.f)
		.BorderImage(FAppStyle::GetBrush("Brushes.Panel"))
		[
			SAssignNew(UniformWrapPanel, SUniformWrapPanel)
		 .HAlign(HAlign_Fill)
		 .SlotPadding(FMargin{4.f, 2.f, 4.f, 2.f })
		];

	if ( NumColumns > 0 )
	{
		UniformWrapPanel->SetNumColumnsOverride( NumColumns );
	}
	if ( CellHeight != TNumericLimits<float>::Min())
	{
		UniformWrapPanel->SetMinDesiredSlotHeight( CellHeight );
		UniformWrapPanel->SetMaxDesiredSlotHeight( CellHeight );
	}
 	
	MainContentWidget = MainContentSBorder;
}
