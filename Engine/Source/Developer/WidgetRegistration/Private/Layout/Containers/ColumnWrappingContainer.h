// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "WidgetContainer.h"

class SBorder;
class SUniformWrapPanel;

/**
* A struct that provides arguments for FColumnWrappingContainers
*/
class FColumnWrappingContainerArgs : public FWidgetContainerArgs
{
public:
	/**
	 * A constructor that initializes the arguments
	 *
	 * @param InCellHeight the height of the cells in this container
	 * @param InNumColumnsOverride the number of columns in the container. If not provided, the number of columns is dynamic,
	 *		  based on the container width and the width of the text of the buttons
	 * @param InIdentifier the identifier for this container
	 * 
	 */
	FColumnWrappingContainerArgs(
	float InCellHeight = TNumericLimits<float>::Min()
	, int32 InNumColumnsOverride = TNumericLimits<float>::Min()
	, FName InIdentifier = "FColumnWrappingContainer" );

	/** the number of columns in the container. If not provided, the number of columns is dynamic,
	 * based on the container width and the width of the text of the buttons */	
	int32 NumColumns;

	/** the height of the cells in this container */
	float CellHeight;
};

/**
 * A container that will provide best fit wrapping for columns, which you can 
 * override if needed.
 */
class FColumnWrappingContainer : public FWidgetContainer
{
public:
	/**
	 * Constructs the FColumnWrappingContainer
	 *
	 * @param Args the parameter object to initialize this container
	 **/
	explicit FColumnWrappingContainer( FColumnWrappingContainerArgs&& Args  );
	
	/**
	 * Constructs the FColumnWrappingContainer
	 *
	 * @param Args the parameter object to initialize this container
	 **/
	explicit FColumnWrappingContainer( const FColumnWrappingContainerArgs& Args = {} );

	/**
	 * Sets the number of columns to the int32 InNumColumns, and returns a reference to this to support chaining
	 * 
	 * @param InNumColumns the number of columns to have in the container
	 * @return a reference to this to support chaining
	 */
	FColumnWrappingContainer& SetNumColumns( const int32& InNumColumns );
	
	/**
	 * Clear the container and any widget content within it
	 */
	virtual void Empty() override;

protected:
	/**
     * Creates and positions within this container the Widget generated by the Builder at index "Index" in ChildBuilderArray 
     * 
     * @param Index the index of the builder in ChildBuilderArray to create and position the widget
     */
	virtual void CreateAndPositionWidgetAtIndex(int32 Index) override;

private:
	/**
	 * Initializes this container
	 */
	void Initialize();

	/** The SBorder that provides the look and feel for this container */
	TSharedPtr<SBorder> MainContentSBorder;

	/**
	 * The SUniformWrapPanel that provides the layout for this container
	 */
	TSharedPtr<SUniformWrapPanel> UniformWrapPanel;

	/** the number of columns in the container. If not provided, the number of columns is dynamic,
	* based on the container width and the width of the text of the buttons */
	int32 NumColumns;

	/** the height of the cells in this container */
	float CellHeight;
};