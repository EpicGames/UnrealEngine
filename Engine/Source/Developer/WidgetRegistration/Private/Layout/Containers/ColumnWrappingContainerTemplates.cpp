// Copyright Epic Games, Inc. All Rights Reserved.

#include "Layout/Containers/ColumnWrappingContainerTemplates.h"

#include "ColumnWrappingContainer.h"

TSharedRef<FColumnWrappingContainer> FColumnWrappingContainerTemplates::GetBestFitColumnsWithSmallCells()
{
	// TODO: extract this out into a size type
	return MakeShared<FColumnWrappingContainer>( FColumnWrappingContainerArgs{ 32 } );			
}

FColumnWrappingContainerTemplates& FColumnWrappingContainerTemplates::Get()
{
	static FColumnWrappingContainerTemplates Presets;
	return Presets;
}