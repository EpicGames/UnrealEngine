// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ColumnWrappingContainer.h"

/**
 * Provides templates for 
 */
struct FColumnWrappingContainerTemplates
{
	/**
	* Gets the singleton instance of FColumnWrappingContainerTemplates
	*/
	static FColumnWrappingContainerTemplates& Get();

	/**
	 * Creates a TSharedRef<FColumnWrappingContainer> that will create columns on a best fit for size basis
	 */
	TSharedRef<FColumnWrappingContainer> GetBestFitColumnsWithSmallCells();
};
