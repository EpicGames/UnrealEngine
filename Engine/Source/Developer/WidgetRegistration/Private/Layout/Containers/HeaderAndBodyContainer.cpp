// Copyright Epic Games, Inc. All Rights Reserved.

#include "Layout/Containers/HeaderAndBodyContainer.h"
#include "Styles/SlateBrushTemplates.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SBox.h"

FHeaderAndBodyContainerArgs::FHeaderAndBodyContainerArgs(
	const FName& InIdentifier,
	const TSharedRef<FSlateBuilder>& InHeader,
	const TSharedRef<FSlateBuilder>& InBody,
	const bool bInIsCollapsible,
	const bool InIsBodyHidden,
	const bool InIsHeaderHidden ) :
	Identifier(InIdentifier)
	, HeaderBuilder (InHeader )
	, BodyBuilder( InBody )
	, bHasToggleButtonToCollapseBody( bInIsCollapsible )
	, bIsBodyHidden( InIsBodyHidden )
	, bIsHeaderHiddenOnCreate( InIsHeaderHidden )
{
}

void FHeaderAndBodyContainer::SetHeader( const TSharedRef<FSlateBuilder>& InHeaderBuilder)
{
	HeaderBuilder = InHeaderBuilder;
}

void FHeaderAndBodyContainer::SetHeader( const TSharedRef<SWidget>& HeaderWidget)
{
	HeaderBuilder = MakeShared<FSlateBuilder>( HeaderWidget );;
}

void FHeaderAndBodyContainer::SetBody( const TSharedRef<FSlateBuilder>& InBodyBuilder)
{
	BodyBuilder = InBodyBuilder;
}

void FHeaderAndBodyContainer::SetBody( const TSharedRef<SWidget>& BodyWidget )
{
	BodyBuilder = MakeShared<FSlateBuilder>( BodyWidget );
}

FHeaderAndBodyContainer::FHeaderAndBodyContainer(const FHeaderAndBodyContainerArgs& Args):
	FSlateBuilder(Args.Identifier)
	, HeaderBuilder( Args.HeaderBuilder )
	, BodyBuilder( Args.BodyBuilder )
	, bHasToggleButtonToCollapseBody( Args.bHasToggleButtonToCollapseBody )
	, bIsBodyHidden( Args.bIsBodyHidden )
	, bIsHeaderHidden( Args.bIsHeaderHiddenOnCreate )
{
}

TSharedPtr<SWidget> FHeaderAndBodyContainer::GenerateWidget()
{

	FCurveSequence RolloutCurve = FCurveSequence(0.0f, 1.0f, ECurveEaseFunction::CubicOut);

	constexpr EStyleColor HeaderForegroundColor = EStyleColor::Foreground;
	const FSlateBrush* ContainerBackgroundBrush = FAppStyle::GetBrush("Brushes.Recessed");

	// TODO: create style class to put this information in
	const FName NoBorderButtonStyle = "NoBorder";
	const float FillHeight1 = 1.0f;
	const FMargin HeaderMargin{ 4.f, 0.f, 0.f, 0.f };
	const float NoPadding = 0.f;

	TSharedPtr<SButton> ExpansionButton;
	
	if ( !bIsBodyHidden )
	{
		RolloutCurve.JumpToEnd();
	}
	
	TSharedRef<SVerticalBox> FullWidgetVerticalBox =
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SAssignNew( HeaderContentSBox, SBox )
			.WidthOverride( FOptionalSize() )
			.HeightOverride( FOptionalSize() )
			[
				SAssignNew( HeaderContentSBorder, SBorder )
				.BorderImage( FAppStyle::GetBrush("Brushes.Dropdown") )
				.Padding( NoPadding )
				[
					SAssignNew( ExpansionButton, SButton )
					.Cursor( EMouseCursor::GrabHand )
					.ButtonStyle(FCoreStyle::Get(), NoBorderButtonStyle )
					.ButtonColorAndOpacity( FStyleColors::Transparent )
					.ContentPadding( NoPadding )
					.ForegroundColor( HeaderForegroundColor )
					[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding( HeaderMargin )
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Center)
						 [
							 bHasToggleButtonToCollapseBody ?
							 SAssignNew( ToggleExpansionImage, SImage )
							 .ColorAndOpacity(FSlateColor::UseForeground()) :
							 SNullWidget::NullWidget
						 ]
						 + SHorizontalBox::Slot()
							 .FillWidth( FillHeight1 )
							 .VAlign(VAlign_Fill)
							  [
								  HeaderBuilder->GenerateWidgetSharedRef()
							  ]
					]
				]
			]
		]
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SNew(SBorder)
			.BorderImage(ContainerBackgroundBrush)
			.Padding(0.f)
			[
				SAssignNew(BodyContentSBox, SBox)
				.WidthOverride( FOptionalSize() )
				.HeightOverride( FOptionalSize() )
				[
					SAssignNew( BodyContentSScrollBox, SScrollBox)
					+ SScrollBox::Slot()
					[
						BodyBuilder->GenerateWidgetSharedRef()
					]
				]
			]
		];

	if ( bHasToggleButtonToCollapseBody )
	{
		ExpansionButton->SetOnClicked( FOnClicked::CreateSP(this, &FHeaderAndBodyContainer::ToggleBodyExpansionState ) );			
	}

	UpdateWidget();		

	return FullWidgetVerticalBox;
}

void FHeaderAndBodyContainer::UpdateWidget()
{
	if ( bIsBodyHidden )
	{
		UpdateToBodyRemovedState();
	}	
	else
	{
		UpdateToBodyAddedState();
	}
	if ( bIsHeaderHidden )
	{ 
		UpdateToHeaderRemovedState();
	}
	else
	{
		UpdateToHeaderAddedState();
	}
}

void FHeaderAndBodyContainer::SetHeaderHidden(bool bInIsHeaderHidden )
{
	bIsHeaderHidden = bInIsHeaderHidden;
	UpdateWidget();
}

FReply FHeaderAndBodyContainer::ToggleBodyExpansionState() 
{
	bIsBodyHidden = !bIsBodyHidden;
	UpdateWidget();
	return FReply::Handled();
}

void FHeaderAndBodyContainer::UpdateToBodyRemovedState()
{
	if ( ToggleExpansionImage.IsValid() )
	{
		const FName CollapsedArrow = "TreeArrow_Collapsed";
		ToggleExpansionImage->SetImage( FCoreStyle::Get().GetBrush( CollapsedArrow ) );
	}
	BodyContentSBox->SetContent( SNullWidget::NullWidget );
	OnBodyAddedOrRemoved.ExecuteIfBound( EBodyLifeCycleEventType::Removed );		
}

void FHeaderAndBodyContainer::UpdateToBodyAddedState()
{
	if ( BodyContentSScrollBox.IsValid() )
	{
		BodyContentSBox->SetContent( BodyContentSScrollBox.ToSharedRef() );
	
		if ( ToggleExpansionImage.IsValid() )
		{
			const FName ExpandedArrow = "TreeArrow_Expanded";
			ToggleExpansionImage->SetImage( FCoreStyle::Get().GetBrush( ExpandedArrow ) );
		}
		OnBodyAddedOrRemoved.ExecuteIfBound( EBodyLifeCycleEventType::Added );		
	}
}

void FHeaderAndBodyContainer::UpdateToHeaderRemovedState()
{
	if ( HeaderContentSBox.IsValid() )
	{
		HeaderContentSBox->SetContent( SNullWidget::NullWidget );		
	}
}

void FHeaderAndBodyContainer::UpdateToHeaderAddedState()
{
	if ( HeaderContentSBorder.IsValid() && HeaderContentSBox.IsValid() )
	{
		HeaderContentSBox->SetContent( HeaderContentSBorder.ToSharedRef() );
	}
}

void FHeaderAndBodyContainer::ResetWidget()
{
}
