// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Layout/SExpandableArea.h"
#include "Layout/Containers/SlateBuilder.h"

class SScrollBox;
class SBox;
class SImage;

/**
 * The parameter objects for a FHeaderAndBodyContainer.
 */
class FHeaderAndBodyContainerArgs
{
public:
	FHeaderAndBodyContainerArgs(
		const FName& InIdentifier = "FHeaderAndBodyContainer",
		const TSharedRef<FSlateBuilder>& InHeader = MakeShared<FSlateBuilder>(),
		const TSharedRef<FSlateBuilder>& InBody = MakeShared<FSlateBuilder>(),
		const bool bInHasToggleButtonToCollapseBody = false,
		const bool bIsBodyHidden = false,
		const bool bIsHeaderHiddenOnCreate = false );

	/** the identifier for the FHeaderAndBodyContainer */
	FName Identifier;

	/** the FSlateBuilder that builds the header of this container */
	TSharedRef<FSlateBuilder> HeaderBuilder;
	
	/** the FSlateBuilder that builds the body of this container */
	TSharedRef<FSlateBuilder> BodyBuilder;

	/** whether or not this container has a button to toggle the body open and closed */
	bool bHasToggleButtonToCollapseBody;

	/** if true, the body is initially hidden */
	bool bIsBodyHidden;

	/** if true, the header is initially hidden */
	bool bIsHeaderHiddenOnCreate;
};

/**
 * A container which provides a header and a body, both of which can be any SWidget.
 */
class  FHeaderAndBodyContainer : public FSlateBuilder
{

public:

	/**
	 * an enum to tell whether the body has been added or removed
	 */
	enum class EBodyLifeCycleEventType
	{
		Added,
		Removed
	};
	
	DECLARE_DELEGATE_OneParam( FOnBodyAddedOrRemoved,  EBodyLifeCycleEventType LifeCycleEventType );

	/**
	 * Sets the FSlateBuilder InHeaderBuilder to build the widget for the header of this container
	 */
	void SetHeader( const TSharedRef<FSlateBuilder>& InHeaderBuilder );

	/**
	 * Converts the SWidget HeaderWidget to an FSlateBuilder which will build the widget for the header of this container
	 */
	void SetHeader( const TSharedRef<SWidget>& HeaderWidget );

	/**
	 * Sets the FSlateBuilder InBodyBuilder to build the widget for the body of this container
	 */
	void SetBody( const TSharedRef<FSlateBuilder>& InBodyBuilder );
	
	/**
	 * Converts the SWidget BodyWidget to an FSlateBuilder which will build the widget for the body of this container
	 */
	void SetBody( const TSharedRef<SWidget>& BodyWidget );

	/**
	 * Constructs this container.
	 *
	 * @param Args the FHeaderAndBodyContainerArgs which will initialize this FHeaderAndBodyContainer
	 */
	FHeaderAndBodyContainer( const FHeaderAndBodyContainerArgs& Args = FHeaderAndBodyContainerArgs() );
	
	FOnBodyAddedOrRemoved OnBodyAddedOrRemoved;

	/**
	 * Generates the widget for this header and body container and returns it
	 */
	virtual TSharedPtr<SWidget> GenerateWidget()  override;

	/** Updates/reloads this widget. This should be called after a consumer has changed any Data in this */
	virtual void UpdateWidget() override;

	/**
	 * Sets the header content to be hidden
	 *
	 * @param bInIsHeaderHidden if true the header is hidden, else it is false
	 */
	void SetHeaderHidden(bool bInIsHeaderHidden );

	/** @return true if the body is empty, else returns false */
	bool IsBodyEmpty()
	{
		return BodyBuilder->IsEmpty();
	}
	
private:
	/**
	 * called when the user toggles the expansion of the body
	 */
	FReply ToggleBodyExpansionState();

	/*** Makes the body visible by adding it */
	void UpdateToBodyAddedState();

	/*** Makes the body invisible by removing it */
	void UpdateToBodyRemovedState();
	
	/*** Makes the header invisible by removing it */
	void UpdateToHeaderRemovedState();

	/*** Makes the header visible by adding it */
	void UpdateToHeaderAddedState();

	/** overriding to hide the method, as it is not needed */
	virtual void ResetWidget() override;

	/** builds the header for this container */
	TSharedRef<FSlateBuilder> HeaderBuilder;

	/** the SBox containing the header  */
	TSharedPtr<SBox> HeaderContentSBox;
	
	/** the SBorder containing the header  */
	TSharedPtr<SBorder> HeaderContentSBorder;

	/** builds the body for this container */
	TSharedRef<FSlateBuilder> BodyBuilder;
	
	/** the SBox containing the body  */
	TSharedPtr<SBox> BodyContentSBox;

	/** the scrollbox for the body */
	TSharedPtr<SScrollBox> BodyContentSScrollBox;

	/** the image for the toggle body button */
	TSharedPtr<SImage> ToggleExpansionImage;

	/** if true, this container has a toggle button to hide and show the body */
	bool bHasToggleButtonToCollapseBody;

	/** if true, the body will not be viewable */
	bool bIsBodyHidden;
	
	/** if true, the header will not be viewable */
	bool bIsHeaderHidden;
};


