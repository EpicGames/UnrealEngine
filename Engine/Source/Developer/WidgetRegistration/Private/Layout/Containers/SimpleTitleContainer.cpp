// Copyright Epic Games, Inc. All Rights Reserved.

#include "Layout/Containers/SimpleTitleContainer.h"

#include "Styling/StyleColors.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

FSimpleTitleContainer::FSimpleTitleContainer(const FSimpleTitleContainerArgs& Args):
	FHeaderAndBodyContainer( Args ) 
	, Title(Args.Title)
{
	TSharedRef<FSlateBuilder> Header = MakeShared<FSlateBuilder>
	(
		SNew (SHorizontalBox )
		+ SHorizontalBox::Slot().HAlign( HAlign_Left ).VAlign( VAlign_Center ).FillWidth( 1.f )
		.Padding(2.f, 4.f, 0.f, 4.f )
		[
			SNew(STextBlock)
			.Text( Title )
		]
	);
	SetHeader( Header );
}

FSimpleTitleContainer::FSimpleTitleContainer( FSimpleTitleContainerArgs&& Args ) :
	FSimpleTitleContainer( Args )
{
}
