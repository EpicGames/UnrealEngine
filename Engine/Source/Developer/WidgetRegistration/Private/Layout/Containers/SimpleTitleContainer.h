// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "HeaderAndBodyContainer.h"

/**
 * a struct to provide arguments to the FSimpleTitleContainer 
 */
class FSimpleTitleContainerArgs : public FHeaderAndBodyContainerArgs
{
public:
	FSimpleTitleContainerArgs(
		FText InTitle = FText::GetEmpty(),
		const FName& InIdentifier = "FSimpleTitleContainer",
		const TSharedRef<FSlateBuilder>& InHeader = MakeShared<FSlateBuilder>(),
		const TSharedRef<FSlateBuilder>& InBody = MakeShared<FSlateBuilder>(),
		const bool bIsExpanded = false ) :
	FHeaderAndBodyContainerArgs(InIdentifier , InHeader, InBody, bIsExpanded )
		, Title( InTitle )    
	{
	}

	FText Title;
};

/**
 * A container providing a simple FText title and a customizable body
 */
class FSimpleTitleContainer : public FHeaderAndBodyContainer
{
public:
	/**
	 * A constructor taking const FSimpleTitleContainerArgs& to initialize it
	 */
	FSimpleTitleContainer( const FSimpleTitleContainerArgs& Args );
	
	/**
	 * A constructor taking const FSimpleTitleContainerArgs& to initialize it
	 */
	FSimpleTitleContainer( FSimpleTitleContainerArgs&& Args );

private:
	/** the title of the container, it will be set in the header */
	FText Title;
};