// Copyright Epic Games, Inc. All Rights Reserved.

#include "Layout/Containers/SlateBuilder.h"

FSlateBuilderArgs::FSlateBuilderArgs(const FName& InName, const TSharedPtr<SWidget> InContent) :
	BuilderKey( UE::DisplayBuilders::FBuilderKeys::Get().None() )
	, Content( InContent )
{
}

FSlateBuilderArgs::FSlateBuilderArgs( const UE::DisplayBuilders::FBuilderKey& InBuilderKey, const TSharedPtr<SWidget> InContent) :
	BuilderKey( InBuilderKey )
	, Content( InContent )
{
}

FSlateBuilder::FSlateBuilder( TSharedPtr<SWidget> Content, FName InIdentifier ) :
	FToolElementRegistrationArgs( InIdentifier )
	, SlateContent( Content )
{
}

FSlateBuilder::FSlateBuilder( UE::DisplayBuilders::FBuilderKey InBuilderKey ) :
	FToolElementRegistrationArgs( InBuilderKey )
{
}

FSlateBuilder::FSlateBuilder( FName InIdentifier ) :
	FToolElementRegistrationArgs( InIdentifier )
{
}

TSharedPtr<SWidget> FSlateBuilder::GenerateWidget()
{
	if ( SlateBuilder.IsValid() )
	{
		SlateContent = SlateBuilder->GenerateWidget();
	}
	return SlateContent;
}

bool FSlateBuilder::IsEmpty()
{
	return !SlateContent.IsValid() || SlateContent == SNullWidget::NullWidget;
}

void FSlateBuilder::Empty()
{
	SlateContent = SNullWidget::NullWidget;
}
