// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ToolElementRegistry.h"
#include "Widgets/SNullWidget.h"

/**
 * An argument object to instantiate the FSlateBuilder
 */
class FSlateBuilderArgs
{
public:
	
	/**
	 * Creates the slate builder parameter object.
	 * 
	 * @param InName the name of the builder
	 * @param InContent the shared pointer to the SWidget that provides the content for this slate builder. If this is nullptr or it is not
	 * provided, this builder will show a NullWidget
	 */
	FSlateBuilderArgs( const FName& InName = "SlateBuilder", const TSharedPtr<SWidget> InContent = SNullWidget::NullWidget.ToSharedPtr() );

	/**
	 * Creates the slate builder parameter object.
	 * 
	 * @param InBuilderKey the FBuilderKey for this builder
	 * @param InContent the shared pointer to the SWidget that provides the content for this slate builder. If this is nullptr or it is not
	 * provided, this builder will show a NullWidget
	 */
	FSlateBuilderArgs( const UE::DisplayBuilders::FBuilderKey& InBuilderKey = UE::DisplayBuilders::FBuilderKeys::Get().None(), const TSharedPtr<SWidget> InContent = nullptr );

	/** the BuilderKey for the arguments  */
	const UE::DisplayBuilders::FBuilderKey BuilderKey;

	/** the shared pointer to the SWidget that provides the content for this slate builder  */
	const TSharedPtr<SWidget> Content;
};

/**
 * A builder which can build raw Slate. This is mostly used for adding raw Slate to Builder Containers, but can be used to build any Slate. It allows
 * raw slate to be used in the context of builder containers, and provides backward compatibility so that Slate and builders can be used in the same
 * context.
 */
class FSlateBuilder : public FToolElementRegistrationArgs
{
public:

	/**
	 * A constructor which takes the TSharedPtr<SWidget> that provides the content for this builder, and the identifier for the builder
	 *
	 * @param InContent the TSharedPtr<SWidget> that provides the content for this builder
	 * @param InIdentifier the identifier for this container
	 */
	explicit FSlateBuilder( TSharedPtr<SWidget> InContent = nullptr, FName InIdentifier = "FSlateBuilder" );

	/**
	 * A constructor providing the identifier for the builder
	 *
	 * @param InIdentifier the identifier for this container
	 */
	explicit FSlateBuilder( FName InIdentifier  );

	/**
	 * A constructor providing an FBuilderKey for the builder
	 *
	 * @param InBuilderKey the FBuilderKey for this FSlateBuilder
	 */
	FSlateBuilder( UE::DisplayBuilders::FBuilderKey InBuilderKey );

	/**
	 * @return the TSharedPtr<SWidget> specified as the content for this FSlateBuilder
	 */
	virtual TSharedPtr<SWidget> GenerateWidget() override;
	
	/**
	 * @return true if the content of this is equivalent to an SNullWidget
	 */
	bool IsEmpty();
	
	/**
	 * Empties the content
	 */
	virtual void Empty();

protected:
	/** The TSharedPtr<SWidget> that is the content for this FSlateBuilder */
	TSharedPtr<SWidget> SlateContent;
	
private:
	
	/** The FSlateBuilder that provides the content for this FSlateBuilder */
	TSharedPtr<FSlateBuilder> SlateBuilder;
};