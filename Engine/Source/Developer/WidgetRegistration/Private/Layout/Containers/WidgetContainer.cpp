// Copyright Epic Games, Inc. All Rights Reserved.

#include "Layout/Containers/WidgetContainer.h"
#include "Layout/Containers/SlateBuilder.h"
#include "Widgets/SNullWidget.h"

FWidgetContainer::FWidgetContainer(const FWidgetContainerArgs Args) :
	FSlateBuilder ( nullptr, Args.Identifier )
{
}

FWidgetContainerArgs::FWidgetContainerArgs( FName InIdentifier) :
	Identifier( InIdentifier )
{
}

FWidgetContainer& FWidgetContainer::AddBuilder(const TSharedRef<FSlateBuilder>& Builder)
{
	ChildBuilderArray.Add( Builder );
	return *this;
}

TSharedPtr<FSlateBuilder> FWidgetContainer::GetBuilderAtIndex(int32 Index)
{
	if ( Index < ChildBuilderArray.Num() )
	{
		return ChildBuilderArray[Index];
	}
	return nullptr;
}

void FWidgetContainer::Empty()
{
	ChildBuilderArray.Empty();
}

FWidgetContainer& FWidgetContainer::AddBuilder( TSharedRef<SWidget>  Widget)
{
	ChildBuilderArray.Add( MakeShared<FSlateBuilder>( Widget ) );
	return *this;
}

TSharedPtr<SWidget> FWidgetContainer::GenerateWidget()
{
	for ( int32 ChildBuilderIndex = 0; ChildBuilderIndex < ChildBuilderArray.Num(); ChildBuilderIndex++)
	{
		CreateAndPositionWidgetAtIndex( ChildBuilderIndex );
	}

	return MainContentWidget.IsValid() ?  MainContentWidget : SNullWidget::NullWidget.ToSharedPtr();
}

void FWidgetContainer::UpdateWidget()
{
	for ( int32 ChildBuilderIndex = 0; ChildBuilderIndex < ChildBuilderArray.Num(); ChildBuilderIndex++)
	{
		ChildBuilderArray[ChildBuilderIndex]->UpdateWidget();
	}
}

FWidgetContainer& FWidgetContainer::SetBuilders( TArray<TSharedRef<FSlateBuilder>> Builders )
{
	Empty();
	
	for ( const TSharedRef<FSlateBuilder>& Builder : Builders )
	{
		ChildBuilderArray.Add( Builder );
	}
	return *this;
}


