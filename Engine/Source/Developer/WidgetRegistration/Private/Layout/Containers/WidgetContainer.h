// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Layout/Containers/SlateBuilder.h"

class SVerticalBox;

/**
 * A struct to provide the arguments for an FWidgetContainer
 */
class FWidgetContainerArgs
{
public:
	/**
	 * The constructor for the widget arguments
	 * 
	 * @param InIdentifier the identifier for this container
	 */
	FWidgetContainerArgs( FName InIdentifier = "FWidgetContainer" );

	
	/** the identifier for this container */
	FName Identifier;
};

/**
 * A Widget Container that contains an array of FSlateBuilders
 */
class FWidgetContainer : public  FSlateBuilder
{
	
public:
	/**
	 * A constructor with const Args which can be used with Presets of extending containers
	 * 
	 * @param Args the arugments to initializes this container
	 */
	explicit FWidgetContainer( const FWidgetContainerArgs Args );

	/**
	 * Generates the container and its children.
	 */
	virtual TSharedPtr<SWidget> GenerateWidget() override;

	/**
	 * Updates the container and its children.
	 */
	virtual void UpdateWidget() override;

	/**
	 * Sets the FSlateBuilders in Builders as the children of this container to provide the contents of this container. If there were previously any builders in
	 * this container, they will be cleared and these Builders will provide the entire content of this container
	 *
	 * @param Widgets the SWidgets that will provide the contents of this container 
	 */
	FWidgetContainer& SetBuilders( TArray<TSharedRef<FSlateBuilder>> Builders );

	/**
	 * Converts "Widget" to an FSlateBuilder and adds it to the existing FSlateBuilders in this container
	 *
	 * @param Widget the widget to add to this container
	 */
	FWidgetContainer& AddBuilder( TSharedRef<SWidget> Widget);

	/**
	 * Adds Builder to the contents
	 *
	 * @param Builder the shared pointer to the FSlateBuilder to add to this container
	 */
	FWidgetContainer& AddBuilder( const TSharedRef<FSlateBuilder>& Builder );
	
	/**
	 * Clear the container and any widget content within it
	 */
	virtual void Empty() override;

protected:
	/**
	 * The SWidget which holds the main content of this container
	 */
	TSharedPtr<SWidget> MainContentWidget;

	/**
	 * @return the Builder at index "Index" in ChildBuilderArray
	 * 
	 * @param Index the index of the builder in ChildBuilderArray to return
	 */
	TSharedPtr<FSlateBuilder> GetBuilderAtIndex( int32 Index );

	/**
	 * Positions the Builder at index "Index" in ChildBuilderArray within this container
	 * 
	 * @param Index the index of the builder in ChildBuilderArray to position
	 */
	virtual void CreateAndPositionWidgetAtIndex( int32 Index ) = 0;

private:
	/**
	 * The array of FSlateBuilders that will be converted to widgets by this container
	 */
	TArray<TSharedRef<FSlateBuilder>> ChildBuilderArray;
};
