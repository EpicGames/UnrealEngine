// Copyright Epic Games, Inc. All Rights Reserved.

#include "Layout/SeparatorBuilder.h"

TSharedPtr<SWidget> FSeparatorBuilder::GenerateWidget()
{
	return SNew( SSeparator )
				.SeparatorImage(Image)
				.Thickness(Size)
				.Orientation(Orientation)
				.Visibility(Visibility)
				.ColorAndOpacity(SlateColor);
}

TSharedRef<SSeparator> FSeparatorBuilder::ToSSeparatorSharedRef()
{
	return StaticCastSharedRef<SSeparator>(GenerateWidget().ToSharedRef());
}

TSharedRef<SSeparator> FSeparatorBuilder::operator*()
{
	return ToSSeparatorSharedRef();
}

FSeparatorBuilder& FSeparatorBuilder::InitializeSize(const FSeparatorSize& NewSize)
{
	Size = NewSize.ToFloat();
	return *this;
}

FSeparatorBuilder& FSeparatorBuilder::SetColor(const EStyleColor& NewColor)
{
	// the overloaded conversion is needed here, as we will want the conversion from EStyleColor to FSlateColor
	SlateColor = NewColor;
	return *this;
}

FSeparatorBuilder::FSeparatorBuilder(EStyleColor InColor, EOrientation InOrientation, FSeparatorSize InSize,
	const FSlateBrush* InImage):
	FToolElementRegistrationArgs(EToolElement::Separator),
	Image(InImage),
	Orientation(InOrientation),
	SlateColor(InColor),
	Size(InSize.ToFloat())
{
}
