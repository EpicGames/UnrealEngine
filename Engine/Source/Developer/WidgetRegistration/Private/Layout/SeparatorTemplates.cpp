// Copyright Epic Games, Inc. All Rights Reserved.

#include "Layout/SeparatorTemplates.h"

FSeparatorBuilder FSeparatorTemplates::SmallVerticalBackgroundNoBorder()
{
	return FSeparatorBuilder{ EStyleColor::Background, Orient_Vertical };
}

FSeparatorBuilder FSeparatorTemplates::SmallHorizontalBackgroundNoBorder()
{
	return FSeparatorBuilder{};
}

FSeparatorBuilder FSeparatorTemplates::SmallHorizontalPanelNoBorder()
{
	return FSeparatorBuilder{ EStyleColor::Panel };
}
