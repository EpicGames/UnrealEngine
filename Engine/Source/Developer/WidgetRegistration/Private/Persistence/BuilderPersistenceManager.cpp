// Copyright Epic Games, Inc. All Rights Reserved.

#include "Persistence/BuilderPersistenceManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BuilderPersistenceManager)

TObjectPtr<UBuilderPersistenceManager> UBuilderPersistenceManager::Instance = nullptr;

namespace UE::DisplayBuilders::BuilderPersistenceManager
{
	FName FavoritesSuffix = "BuilderFavorites";
	FName ShowButtonLabelsSuffix = "ShowButtonLabels";
}

void UBuilderPersistenceManager::Initialize()
{
	if(!Instance)
	{
		Instance = NewObject<UBuilderPersistenceManager>(); 
		Instance->AddToRoot();

		Instance->LoadEditorConfig();
	}
}

void UBuilderPersistenceManager::ShutDown()
{
	if ( UObjectInitialized() )
	{
		Instance->RemoveFromRoot();
	}
	Instance = nullptr;
}

TArray<FName> UBuilderPersistenceManager::GetFavoritesNames(const UE::DisplayBuilders::FBuilderKey& Key)
{
	return GetPersistedArrayOfNames( Key, UE::DisplayBuilders::BuilderPersistenceManager::FavoritesSuffix );
}

void UBuilderPersistenceManager::PersistFavoritesNames(const UE::DisplayBuilders::FBuilderKey& Key, TArray<FName>& Favorites)
{
	return PersistArrayOfNames(Key, UE::DisplayBuilders::BuilderPersistenceManager::FavoritesSuffix, Favorites);
}

bool UBuilderPersistenceManager::GetShowButtonLabels( const UE::DisplayBuilders::FBuilderKey& Key, bool bDefaultValue )
{
	return GetPersistedBool( Key, UE::DisplayBuilders::BuilderPersistenceManager::ShowButtonLabelsSuffix, bDefaultValue );
}

void UBuilderPersistenceManager::PersistShowButtonLabels( const UE::DisplayBuilders::FBuilderKey& Key, bool bValue )
{
	return PersistBool( Key, UE::DisplayBuilders::BuilderPersistenceManager::ShowButtonLabelsSuffix, bValue );
}

TArray<FName> UBuilderPersistenceManager::GetPersistedArrayOfNames( const UE::DisplayBuilders::FBuilderKey& Key, FName PersistenceKeySuffix )
{
	if ( !Key.IsNone()  && !PersistenceKeySuffix.IsNone() )
	{
		if ( const FPersistedNameArray* Settings =  SavedNameToPersistedFNameArrayMap.Find( Key.GetKeyWithSuffix( PersistenceKeySuffix ) ) )
		{
			return Settings->ArrayOfNamesToPersist;
		}
	}

	return {};
}

void UBuilderPersistenceManager::PersistArrayOfNames( const UE::DisplayBuilders::FBuilderKey& Key, FName PersistenceKeySuffix, TArray<FName>& ArrayOfNamesToPersist )
{
	if (PersistenceKeySuffix.IsNone())
	{
		return;
	}

	FPersistedNameArray& Settings =  SavedNameToPersistedFNameArrayMap.Add( Key.GetKeyWithSuffix( PersistenceKeySuffix ) );

	Settings.ArrayOfNamesToPersist = ArrayOfNamesToPersist;;
	SaveEditorConfig();
}

bool UBuilderPersistenceManager::GetPersistedBool( const UE::DisplayBuilders::FBuilderKey& Key, FName PersistenceKeySuffix, bool bDefaultValue )
{
	if ( !Key.IsNone()  && !PersistenceKeySuffix.IsNone() )
	{
		if ( const FPersistedBool* BoolSettings =  SavedNameToPersistedBoolMap.Find( Key.GetKeyWithSuffix( PersistenceKeySuffix ) ) )
		{
			return BoolSettings->PersistedBool;
		}
	}

	return bDefaultValue;
}

void UBuilderPersistenceManager::PersistBool( const UE::DisplayBuilders::FBuilderKey& Key, FName PersistenceKeySuffix, bool bValue )
{
	if (PersistenceKeySuffix.IsNone())
	{
		return;
	}

	FPersistedBool& Settings =  SavedNameToPersistedBoolMap.Add( Key.GetKeyWithSuffix( PersistenceKeySuffix ) );
	Settings.PersistedBool = bValue;
	SaveEditorConfig();
}
