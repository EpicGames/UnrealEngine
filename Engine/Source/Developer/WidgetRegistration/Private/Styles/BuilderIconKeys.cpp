// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuilderIconKeys.h"

#include "BuilderIconSizeKeys.h"
#include "BuilderStyleManager.h"

// folder path information
namespace UE::DisplayBuilders::BuilderIconsKeys::Paths
{
	const TCHAR Slash = '/';
	const TCHAR* Icons_DataVisualization = TEXT("Icons/DataVisualization");
}

// icon name information
namespace UE::DisplayBuilders::BuilderIconsKeys::Icons
{
	const FName ZeroStateDefault = "ZeroStateDefault";
	const FName ZeroStateFavorites = "ZeroStateFavorites";
}

const FBuilderIconKeys& FBuilderIconKeys::Get()
{
	static const FBuilderIconKeys Keys;
	return Keys;
}

const FBuilderIconKey& FBuilderIconKeys::ZeroStateDefaultMedium() const
{
	static const FBuilderIconKey Key( UE::DisplayBuilders::BuilderIconsKeys::Paths::Icons_DataVisualization,UE::DisplayBuilders::BuilderIconsKeys:: Icons::ZeroStateDefault, FBuilderIconSizeKeys::Get().Medium() );
	return Key;
}

const FBuilderIconKey& FBuilderIconKeys::ZeroStateFavoritesMedium() const
{
	static const FBuilderIconKey Key( UE::DisplayBuilders::BuilderIconsKeys::Paths::Icons_DataVisualization,UE::DisplayBuilders::BuilderIconsKeys:: Icons::ZeroStateFavorites, FBuilderIconSizeKeys::Get().Medium() );
	return Key;
}

FSlateIcon FBuilderIconKey::GetSlateIcon() const
{
	return  FSlateIcon( FBuilderStyleManager::Get().GetStyleSetName(), FileNameWithoutExtension );
}

FBuilderIconKey::FBuilderIconKey( FString InRelativePathToContainingFolder, const FName InName, const FBuilderIconSizeKey& InSizeKey):
	RelativePathToContainingFolder( InRelativePathToContainingFolder )
	, Name( InName )
	, SizeKey( InSizeKey )
	, FileNameWithoutExtension( Name.ToString() + InSizeKey.Name.ToString() )
	, RelativePathToFileWithoutExtension( RelativePathToContainingFolder + UE::DisplayBuilders::BuilderIconsKeys::Paths::Slash + FileNameWithoutExtension.ToString() )
{
}

FBuilderIconKeys::FBuilderIconKeys() 
{
}
