// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BuilderIconSizeKeys.h"
#include "BuilderStyleManager.h"
#include "Textures/SlateIcon.h"

class FBuilderIconSizeKeys;
class FBuilderIconSizeKey;
class FBuilderIconKeys;

/**
* Provides keys for display builder Icons 
*/
class FBuilderIconKey
{
public:

	/** initializes the slate icon. This must be called after the the icon is registered by the Builder Style */
	void InitializeIcon()
	{
		SlateIcon =  FSlateIcon( FBuilderStyleManager::Get().GetStyleSetName(), FileNameWithoutExtension );
	}

	/**
	* The FString providing the relative path to the icon 
	*/
	const FString RelativePathToContainingFolder;

	/**
	* The name of the Icon 
	*/
	const FName Name;
	
	/**
	 * The FBuilderIconSizeKey that provides the size of the Icon
	 */
	const FBuilderIconSizeKey& SizeKey;

	/**
	* The name of the Icon file, not including relative path or extension
	*/
	const FName FileNameWithoutExtension;

	/**
	 * the relative path to the file for the icon. This includes both the relative path of directories and the name of the file for the icon,
	 * but not the file extension
	 */
	const FString RelativePathToFileWithoutExtension;
	

	/**
	 * @return the FSlateIcon for this FBuilderIconKey
	 */
	FSlateIcon GetSlateIcon() const;

private:

	friend FBuilderIconKeys;
	
	FSlateIcon SlateIcon;
	/**
	* The constructor which takes an FName which provides the identifier for this BuilderIconKey
	*
	* @param InName the FName which provides the identifier for this BuilderIconKey
	* @param InRelativePathToContainingFolder the relative path to the containing folder
	* @param InSizeKey the icon size key for this 
	*/
	 FBuilderIconKey( const FString InRelativePathToContainingFolder, const FName InName, const FBuilderIconSizeKey& InSizeKey );
};


/**
 * Keys for builder icons
 */
class FBuilderIconKeys
{
public:
	/**
	 * Gets the FBuilderIconKeys singleton
	 */
	static const FBuilderIconKeys& Get();

	/** An Icon for the default Zero State view. This shows a default icon that denotes that there are no items available */
	const FBuilderIconKey& ZeroStateDefaultMedium() const;
	
	/** An Icon for the favorites Zero State view. This shows a zero state favorites icon that denotes that there are no favorites available */
	const FBuilderIconKey& ZeroStateFavoritesMedium() const;
	
private:
	/** private constructor as these keys should map to icons that we include with builder content */
	FBuilderIconKeys();
};
