// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "UObject/NameTypes.h"
#include "Math/Vector2D.h"

class FBuilderIconSizeKeys;

/**
* Provides identifiers and values for icon sizes 
*/
class FBuilderIconSizeKey
{
public:
	/**
	 * default constructor that initializes the key
	 */
	explicit FBuilderIconSizeKey( FString RelativePathToContainingFolder );

	/**
	* The name of the Icon 
	*/
	const FName Name;

	/**
	 * The Size of the Icon
	 */
	const FVector2D Size;
 	
	
private:

	friend FBuilderIconSizeKeys;
	
	/**
	* The constructor which takes an FName which provides the identifier for this BuilderIconSizeKey
	*
	* @param InName the FName which provides the identifier for this BuilderIconSizeKey
	* @param InSize the FVector2D size of the icon
	*/
	 FBuilderIconSizeKey( const FName InName, const FVector2D InSize ) :
		Name( InName )
		, Size( InSize )
	{
	}
};

/**
 * FBuilderIconSizeKeys provides sizes for builder icons
 */
class FBuilderIconSizeKeys
{
public:
	static const FBuilderIconSizeKeys& Get();

	const FBuilderIconSizeKey& ExtraSmall() const
	{
		static const FBuilderIconSizeKey Key( "ExtraSmall", FVector2D(16, 16) );
		return Key;
	}

	const FBuilderIconSizeKey& Small() const
	{
		static const FBuilderIconSizeKey Key( "ExtraSmall", FVector2D(20, 20) );
		return Key;
	}

	const FBuilderIconSizeKey& Medium() const
	{
		static const FBuilderIconSizeKey Key( "ExtraSmall", FVector2D(40, 40) );
		return Key;
	}
	
private:
	FBuilderIconSizeKeys();
};

inline const FBuilderIconSizeKeys& FBuilderIconSizeKeys::Get()
{
	static const FBuilderIconSizeKeys Keys;
	return Keys;
}

inline FBuilderIconSizeKeys::FBuilderIconSizeKeys()
{
}
