// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuilderStyleManager.h"

#include "BuilderIconKeys.h"
#include "Brushes/SlateImageBrush.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/ToolBarStyle.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/StyleColors.h"

void FBuilderStyleManager::RegisterSlateIcon( FBuilderIconKey Key )
{
	Set( Key.FileNameWithoutExtension, new CORE_IMAGE_BRUSH_SVG( Key.RelativePathToFileWithoutExtension, Key.SizeKey.Size ));
	Key.InitializeIcon();
}

FBuilderStyleManager::FBuilderStyleManager()
	: FSlateStyleSet(TEXT("Builder"))
{

	SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate/Builders"));
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FBuilderStyleManager::~FBuilderStyleManager()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}
