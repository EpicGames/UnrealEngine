// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class FBuilderIconKey;

/**
 * Provides styling utilities and information for builders.
 */
class FBuilderStyleManager final : public FSlateStyleSet
{
public:
	static FBuilderStyleManager& Get()
	{
		static FBuilderStyleManager Instance;
		return Instance;
	}

	/**
	 * Given Key, registers the FSlateIcon for it
	 * 
	 * @param Key the FBuilderIconKey to register the FSlateIcon for
	 */
	void RegisterSlateIcon( const FBuilderIconKey Key );
	
	FBuilderStyleManager();
	
	virtual ~FBuilderStyleManager() override;
};
