// Copyright Epic Games, Inc. All Rights Reserved.

#include "Styles/SlateBrushTemplates.h"
#include "Brushes/SlateColorBrush.h"
#include "Styling/CoreStyle.h"
#include "Styling/StyleColors.h"


PRAGMA_DISABLE_DEPRECATION_WARNINGS

const FSlateBrush* FSlateBrushTemplates::DragHandle()
{
	const FSlateBrush* Brush = FCoreStyle::Get().GetBrush("VerticalBoxDragIndicatorShort");
	return Brush;
}

const FSlateBrush* FSlateBrushTemplates::ThinLineHorizontal()
{
	return FAppStyle::GetBrush( "ThinLine.Horizontal");		
}

const FSlateBrush* FSlateBrushTemplates::Dropdown()
{
	static const FSlateColorBrush ColorBrush{ FStyleColors::Dropdown };
	return &ColorBrush;
}

const FSlateBrush* FSlateBrushTemplates::Transparent()
{
	static const FSlateColorBrush ColorBrush{ FLinearColor::Transparent };
	return &ColorBrush;
}

const FSlateBrush* FSlateBrushTemplates::Panel()
{
	static const FSlateColorBrush ColorBrush{ FStyleColors::Panel };
	return &ColorBrush;
}

const FSlateBrush* FSlateBrushTemplates::Recessed()
{
	static const FSlateColorBrush ColorBrush{ FStyleColors::Recessed };
	return &ColorBrush;
}

const FSlateBrush* FSlateBrushTemplates::GetBrushWithColor(EStyleColor Color)
{
	const TUniquePtr<FSlateColorBrush>& ColorBrushPtr = EStyleColorToSlateBrushMap.FindOrAdd(Color, MakeUnique<FSlateColorBrush>(Color));

	return ColorBrushPtr.Get();
}

FSlateBrushTemplates& FSlateBrushTemplates::Get()
{
	static FSlateBrushTemplates Templates;
	return Templates;
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS
