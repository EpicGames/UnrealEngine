// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolElementRegistry.h"

#include "Widgets/SNullWidget.h"

void FToolElementRegistrationArgs::UpdateWidget()
{
	
}

void FToolElementRegistrationArgs::ResetWidget()
{
	
}

FToolElementRegistrationArgs::FToolElementRegistrationArgs(
	EToolElement InToolElementType) :
	ToolElementType(InToolElementType),
	StyleClassName("Section")
{
}

FToolElementRegistrationArgs::FToolElementRegistrationArgs( FName InStyleClassName ) :
	ToolElementType(EToolElement::Section),
	StyleClassName(InStyleClassName)
{
}

FToolElementRegistrationArgs::FToolElementRegistrationArgs( UE::DisplayBuilders::FBuilderKey InBuilderKey ) :
	ToolElementType(EToolElement::Section)
	, StyleClassName( InBuilderKey.ToName() )
	, BuilderKey( InBuilderKey )
{
}

TSharedPtr<SWidget> FToolElementRegistrationArgs::GenerateWidget()
{
	return nullptr;
}

TSharedRef<SWidget> FToolElementRegistrationArgs::GenerateWidgetSharedRef()
{
	if (TSharedPtr<SWidget> Widget = GenerateWidget())
	{
		return Widget.ToSharedRef();
	}
	return SNullWidget::NullWidget;
}

FToolElementRegistrationKey::FToolElementRegistrationKey(FName InName, EToolElement InToolElementType ) :
		Name(InName), ToolElementType(InToolElementType)
{
	FString KeyStr = Name.ToString().AppendChar(' ');
	KeyStr.AppendInt( static_cast<int32>(ToolElementType));
	KeyString = KeyStr;
}

FString FToolElementRegistrationKey::GetKeyString()
{
	return KeyString;	
}

FToolElement::FToolElement(
		const FName InName,
		TSharedRef<FToolElementRegistrationArgs> InBuilderArgs) :
		FToolElementRegistrationKey(
			InName,
			InBuilderArgs->ToolElementType),
		RegistrationArgs(InBuilderArgs)
{
}

void FToolElement::SetRegistrationArgs(TSharedRef<FToolElementRegistrationArgs> InRegistrationArgs)
{
	RegistrationArgs = InRegistrationArgs;
}

TSharedRef<SWidget> FToolElement::GenerateWidget()
{
	return RegistrationArgs->GenerateWidget().ToSharedRef();
}

FToolElementRegistry::FToolElementRegistry()
{
	ToolElementKeyToToolElementMap.Reset();
}

FToolElementRegistry& FToolElementRegistry::Get()
{
	static FToolElementRegistry Registry;
	return Registry;
}

TSharedPtr<FToolElement> FToolElementRegistry::GetToolElementSP(FToolElementRegistrationKey& ToolElementKey)
{
	TSharedPtr<FToolElement>* Element = ToolElementKeyToToolElementMap.Find(*(ToolElementKey.GetKeyString()));
	return Element != nullptr ? *Element : nullptr;
}

TSharedRef< SWidget > FToolElementRegistry::GenerateWidget(
		TSharedRef<FToolElementRegistrationKey> ToolElementKeySR,
		TSharedPtr<FToolElementRegistrationArgs> RegistrationArgsSP,
		bool bUpdateRegistrationArgs)
{
	if (const TSharedPtr<FToolElement> Element = GetToolElementSP(ToolElementKeySR.Get()))
	{
		const bool bHasNewRegistrationArgs = RegistrationArgsSP != nullptr;
		const TSharedRef<FToolElementRegistrationArgs> TempArgsSR = Element->RegistrationArgs;
			
		if (bHasNewRegistrationArgs)
		{
			Element->RegistrationArgs = RegistrationArgsSP.ToSharedRef();		
		}
		const TSharedPtr<SWidget> Widget = Element->RegistrationArgs->GenerateWidget();

		if ( !bUpdateRegistrationArgs && bHasNewRegistrationArgs )
		{
			Element->RegistrationArgs = TempArgsSR;
		}
		
		return Widget != nullptr &&  Widget.IsValid() ? Widget.ToSharedRef() : SNullWidget::NullWidget;
	}

	return SNullWidget::NullWidget;
}


void FToolElementRegistry::RegisterElement(const TSharedRef<FToolElement> ToolElement)
{
	ToolElementKeyToToolElementMap.Add(*ToolElement->GetKeyString(), ToolElement);
}

void FToolElementRegistry::UnregisterElement(const TSharedRef<FToolElement> ToolElement)
{
	ToolElementKeyToToolElementMap.Remove(*ToolElement->GetKeyString());
}
