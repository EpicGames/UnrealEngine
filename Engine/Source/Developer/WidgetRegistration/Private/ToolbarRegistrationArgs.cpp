// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolbarRegistrationArgs.h"


FToolbarRegistrationArgs::FToolbarRegistrationArgs(TSharedRef<FToolBarBuilder> InToolBarBuilder) :
		FToolElementRegistrationArgs(EToolElement::Toolbar),
		ToolBarBuilder(InToolBarBuilder)
{
		
}

TSharedPtr<SWidget> FToolbarRegistrationArgs::GenerateWidget()
{
	return ToolBarBuilder->MakeWidget();
}