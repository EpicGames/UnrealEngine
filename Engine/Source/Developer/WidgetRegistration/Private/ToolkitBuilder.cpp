// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolkitBuilder.h"

#include "IDetailsView.h"
#include "Widgets/SBoxPanel.h"
#include "SPrimaryButton.h"
#include "ToolbarRegistrationArgs.h"
#include "ToolElementRegistry.h"
#include "ToolkitStyle.h"
#include "Framework/Commands/UICommandList.h"
#include "Layout/SeparatorBuilder.h"
#include "Layout/SeparatorTemplates.h"
#include "Styling/StyleColors.h"
#include "Layout/CategoryDrivenContentBuilderBase.h"

#define LOCTEXT_NAMESPACE "ToolkitBuilder"

bool FEditablePalette::IsInPalette(const FName CommandName) const
{
	return this->PaletteCommandNameArray.Contains(CommandName.ToString());
}

TArray<FString> FEditablePalette::GetPaletteCommandNames() const
{
	return PaletteCommandNameArray;
}

void FEditablePalette::AddCommandToPalette(const FString CommandNameString)
{
	PaletteCommandNameArray.Add(CommandNameString);

	SaveToConfig();
	
	OnPaletteEdited.ExecuteIfBound();
}

void FEditablePalette::RemoveCommandFromPalette(const FString CommandNameString)
{
	PaletteCommandNameArray.Remove(CommandNameString);

	SaveToConfig();
	
	OnPaletteEdited.ExecuteIfBound();
}

void FEditablePalette::SaveToConfig()
{
	if(GetConfigManager.IsBound())
	{
		IEditableToolPaletteConfigManager* ConfigManager = GetConfigManager.Execute();
		if(ConfigManager)
		{
			if(FEditableToolPaletteSettings* Config = ConfigManager->GetMutablePaletteConfig(EditablePaletteName))
			{
				Config->PaletteCommandNames = PaletteCommandNameArray;
				ConfigManager->SavePaletteConfig(EditablePaletteName);
			}
		}
	}
}

void FEditablePalette::LoadFromConfig()
{
	if(GetConfigManager.IsBound())
	{
		if(IEditableToolPaletteConfigManager* ConfigManager = GetConfigManager.Execute())
		{
			if(FEditableToolPaletteSettings* Config = ConfigManager->GetMutablePaletteConfig(EditablePaletteName))
			{
				PaletteCommandNameArray = Config->PaletteCommandNames;
			}
		}
	}
}

FToolkitBuilderArgs::FToolkitBuilderArgs(const FName InToolbarCustomizationName):
	FCategoryDrivenContentBuilderArgs(InToolbarCustomizationName)
	, ToolbarCustomizationName(InToolbarCustomizationName)
{
}

FEditablePalette::FEditablePalette(TSharedPtr<FUICommandInfo> InLoadToolPaletteAction,
                                   TSharedPtr<FUICommandInfo> InAddToPaletteAction,
                                   TSharedPtr<FUICommandInfo> InRemoveFromPaletteAction,
                                   FName InEditablePaletteName,
                                   FGetEditableToolPaletteConfigManager InGetConfigManager) :
                                                                                            FToolPalette(InLoadToolPaletteAction, {}),
                                                                                            AddToPaletteAction(InAddToPaletteAction),
                                                                                            RemoveFromPaletteAction(InRemoveFromPaletteAction),
                                                                                            EditablePaletteName(InEditablePaletteName),
                                                                                            GetConfigManager(InGetConfigManager)
{
	LoadFromConfig();
}


FToolkitBuilder::FToolkitBuilder(FToolkitBuilderArgs& Args):
                                                                 FCategoryDrivenContentBuilderBase(Args),
                                                                 ToolkitCommandList(Args.ToolkitCommandList),
                                                                 SelectedCategoryTitleVisibility(Args.SelectedCategoryTitleVisibility),
                                                                 ToolkitSections(Args.ToolkitSections)
{
	const bool bInitLoadToolPaletteMap = true;
	InitializeCategoryToolbar( bInitLoadToolPaletteMap );
	ResetWidget();
}

FToolkitBuilder::~FToolkitBuilder()
{
	for (const TSharedRef<FToolElement>& PaletteElement : ToolPaletteElementArray)
	{
		ToolRegistry.UnregisterElement(PaletteElement);
	}
}

FToolkitBuilder::FToolkitBuilder(
	FName InToolbarCustomizationName,
	TSharedPtr<FUICommandList> InToolkitCommandList,
	TSharedPtr<FToolkitSections> InToolkitSections) :
	FCategoryDrivenContentBuilderBase("FToolkitBuilder"),
	ToolbarCustomizationName(InToolbarCustomizationName),
	ToolkitCommandList(InToolkitCommandList),
	ToolkitSections(InToolkitSections)
{
	ResetWidget();
}

void FToolkitBuilder::GetCommandsForEditablePalette(TSharedRef<FEditablePalette> EditablePalette, TArray<TSharedPtr<const FUICommandInfo>>& OutCommands)
{
	TArray<FString> CommandNames = EditablePalette->GetPaletteCommandNames();

	for(const FString& CommandName : CommandNames)
	{
		if(TSharedPtr<const FUICommandInfo>* FoundCommand = PaletteCommandInfos.Find(CommandName))
		{
			if(*FoundCommand)
			{
				OutCommands.Add(*FoundCommand);
			}
			else
			{
				UE_LOGF(LogTemp, Display, "%ls: Could not find Favorited Tool %ls", *ToolbarCustomizationName.ToString(), *CommandName);
			}
		}
	}
}

FName FToolkitBuilder::GetActivePaletteName() const
{
	if (ActivePalette.IsValid() && ActivePalette->LoadToolPaletteAction.IsValid())
	{
		return ActivePalette->LoadToolPaletteAction->GetCommandName();
	}
	return NAME_None;
}

void FToolkitBuilder::AddPalette(TSharedPtr<FEditablePalette> Palette)
{
	Palette->OnPaletteEdited.BindSP(this, &FToolkitBuilder::OnEditablePaletteEdited, Palette.ToSharedRef());
	EditablePalettesArray.Add(Palette.ToSharedRef());
	AddPalette(StaticCastSharedRef<FToolPalette>(Palette.ToSharedRef()) );
}

void FToolkitBuilder::AddPalette(TSharedPtr<FToolPalette> Palette)
{
	for (TSharedRef<FButtonArgs> Button : Palette->PaletteActions)
	{
		PaletteCommandNameToButtonArgsMap.Add(Button->Command->GetCommandName().ToString(), Button);
		PaletteCommandInfos.Add(Button->Command->GetCommandName().ToString(), Button->Command);
	}
	const FName CommandName = Palette->LoadToolPaletteAction->GetCommandName();
	LoadCommandNameToToolPaletteMap.Add(CommandName.ToString(), Palette);
	LoadCommandArray.Add( CommandName ); 

	LoadToolPaletteCommandList->MapAction(
				Palette->LoadToolPaletteAction,
				FExecuteAction::CreateSP(SharedThis(this),
				&FToolkitBuilder::TogglePalette,
				Palette),
				FCanExecuteAction(),
				FGetActionCheckState::CreateSP(SharedThis(this),
					&FToolkitBuilder::IsActiveToolPalette,
					Palette->LoadToolPaletteAction->GetCommandName())
									 	);
	LoadPaletteToolBarBuilder->AddToolBarButton(Palette->LoadToolPaletteAction);
}

ECheckBoxState FToolkitBuilder::IsActiveToolPalette(FName CommandName) const
{
	return (ActivePalette &&
			ActivePalette->LoadToolPaletteAction->GetCommandName() == CommandName) ?
				ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FToolkitBuilder::UpdateEditablePalette(TSharedRef<FEditablePalette> Palette)
{
	Palette->PaletteActions.Empty();

	TArray<FString> PaletteComandNameArray = Palette->GetPaletteCommandNames();

	for (const FString& Key : PaletteComandNameArray)
	{
		if (const TSharedPtr<FButtonArgs>* FoundButton = PaletteCommandNameToButtonArgsMap.Find(Key))
		{
			const TSharedRef<FButtonArgs> Button = (*FoundButton).ToSharedRef();
			Palette->PaletteActions.Add(Button);
		}
	}
}

void FToolkitBuilder::OnEditablePaletteEdited(TSharedRef<FEditablePalette> EditablePalette)
{
	UpdateEditablePalette(EditablePalette);

	// if the active Palette is the Palette to which we are toggling an action,
	// recreate it to load the new state after the toggle
	if ( ActivePalette == EditablePalette )
	{
		CreatePalette(EditablePalette);
	}
}

void FToolkitBuilder::UpdateWidget()
{
	for (const TSharedRef<FEditablePalette>& EditablePalette : EditablePalettesArray)
	{
		UpdateEditablePalette(EditablePalette);		
	}
}

void FToolkitBuilder::ToggleCommandInPalette(TSharedRef<FEditablePalette> Palette, FString CommandNameString)
{
	if (!Palette->IsInPalette(FName(CommandNameString)))
	{	
		Palette->AddCommandToPalette(CommandNameString);
	}
	else
	{
		Palette->RemoveCommandFromPalette(CommandNameString);		
	}
}

bool FToolkitBuilder::HasActivePalette() const
{
	return ActivePalette != nullptr;
}

void FToolkitBuilder::InitializeCategoryToolbar()
{
	const bool bInitializeToolbar = false;
	InitializeCategoryToolbar( bInitializeToolbar );
}

void FToolkitBuilder::InitializeCategoryToolbar(bool bInitLoadToolPaletteMap)
{

	const bool bForceSmallIcons = true; 
	Style = FToolkitStyle::Get().GetWidgetStyle<FToolkitWidgetStyle>("FToolkitWidgetStyle");
	LoadToolPaletteCommandList = MakeShareable(new FUICommandList);
	LoadPaletteToolBarBuilder = MakeShared<FVerticalToolBarBuilder>(LoadToolPaletteCommandList, FMultiBoxCustomization::None, TSharedPtr<FExtender>(), bForceSmallIcons);

	LoadPaletteToolBarBuilder->SetLabelVisibility( CategoryButtonLabelVisibility );
	const FName StyleName = GetCategoryToolBarStyleName(); 
	LoadPaletteToolBarBuilder->SetStyle(&FAppStyle::Get(), StyleName );
	
	EditablePalettesArray.Reset();

	if (bInitLoadToolPaletteMap)
	{
		LoadCommandNameToPaletteToolbarBuilderMap.Reset();
	}
	else
	{
		TArray<FString> Keys;
		LoadCommandNameToToolPaletteMap.GetKeys(Keys);
		for ( const FString& Name : Keys)
		{
			const TSharedPtr<FToolPalette>* Palette = LoadCommandNameToToolPaletteMap.Find(Name);
			if (Palette)
			{
				AddPalette(*Palette);			
			}
		}
	}

}

void FToolkitBuilder::SetCategoryToolbarVisibility(EVisibility Visibility)
{
	CategoryToolbarVisibility = Visibility;
}

void FToolkitBuilder::TogglePalette(TSharedPtr<FToolPalette> Palette)
{
	const FName CommandName = Palette->LoadToolPaletteAction->GetCommandName();
	
	if (ActivePalette && ActivePalette->LoadToolPaletteAction->GetCommandName() == CommandName)
	{
		switch (CategoryReclickBehavior)
		{
		case ECategoryReclickBehavior::NoEffect:
			return;
		case ECategoryReclickBehavior::ToggleOff:
			ActivePalette = nullptr;
			ResetToolPaletteWidget();
			return;
		case ECategoryReclickBehavior::TreatAsChanged:
			// Fall through to the other handling
			break;
		default:
			ensure(false);
		}
	}
	
	CreatePalette(Palette);

	// active palette has changed
	OnActivePaletteChanged.Broadcast();
}

void FToolkitBuilder::CreatePalette(TSharedPtr<FToolPalette> Palette)
{
	if (!Palette)
	{
		return;
	}
	
	const FName CommandName = Palette->LoadToolPaletteAction->GetCommandName();
	ActivePalette = Palette;
	ResetToolPaletteWidget();
	const TSharedPtr<FSlimHorizontalUniformToolBarBuilder> PaletteToolbarBuilder =
		MakeShareable(new FSlimHorizontalUniformToolBarBuilder(ToolkitCommandList,
			FMultiBoxCustomization(ToolbarCustomizationName)));

	const TSharedPtr<FToolbarRegistrationArgs> RegistrationArgs = MakeShareable<FToolbarRegistrationArgs>(
		new FToolbarRegistrationArgs(PaletteToolbarBuilder.ToSharedRef()));
	FToolElementRegistrationKey Key = FToolElementRegistrationKey(CommandName, EToolElement::Toolbar);
	TSharedPtr<FToolElement> Element = ToolRegistry.GetToolElementSP(Key);
	
	if (!Element)
	{
		Element = MakeShareable(
				new FToolElement(Palette->LoadToolPaletteAction->GetCommandName(),
				RegistrationArgs.ToSharedRef()));
		ToolRegistry.RegisterElement(Element.ToSharedRef());
	}
	
	Element->SetRegistrationArgs(RegistrationArgs.ToSharedRef());
	
	LoadCommandNameToPaletteToolbarBuilderMap.Add(Palette->LoadToolPaletteAction->GetCommandName(), PaletteToolbarBuilder.ToSharedRef());
	ToolPaletteElementArray.Add(Element.ToSharedRef());
	
	PaletteToolbarBuilder->SetStyle(&FAppStyle::Get(), "SlimPaletteToolBar");

	for (TSharedRef<FButtonArgs> PaletteButton : Palette->PaletteActions)
	{
		PaletteButton->CommandList = ToolkitCommandList;
		PaletteButton->UserInterfaceActionType = (PaletteButton->UserInterfaceActionType != EUserInterfaceActionType::None) ?
														PaletteButton->UserInterfaceActionType :
														EUserInterfaceActionType::ToggleButton;
		PaletteButton->OnGetMenuContent.BindSP(SharedThis(this),
			&FToolkitBuilder::GetContextMenuContent,
			PaletteButton->Command->GetCommandName());
		PaletteToolbarBuilder->AddToolBarButton(PaletteButton.Get());
	}
	CreatePaletteWidget(*Palette.Get(), *Element.Get());
	LoadPaletteToolBarBuilder->SetLastSelectedCommandIndex( LoadCommandArray.IndexOfByKey(Palette->LoadToolPaletteAction->GetCommandName()) );
}

void FToolkitBuilder::CreatePaletteWidget(FToolPalette& Palette, FToolElement& Element)
{
	ToolPaletteWidget->AddSlot()
		.HAlign(HAlign_Fill)
		.Padding(0.f)
		.FillHeight(1.0f)
		[

		SNew(SBorder)
		.Padding(Style.TitlePadding)
		.VAlign(VAlign_Center)
		.Visibility( SelectedCategoryTitleVisibility )
		.BorderImage(&Style.TitleBackgroundBrush)
		.HAlign(HAlign_Left)
		[

			SNew(STextBlock)
					.Justification(ETextJustify::Left)
					.Font(Style.TitleFont)
					.Text(Palette.LoadToolPaletteAction->GetLabel())
					.ColorAndOpacity(Style.TitleForegroundColor)
		]];
	
	TSharedPtr<SWidget> PaletteButtonsWidget = Element.GenerateWidget();
	PaletteButtonsWidget->SetVisibility(TAttribute<EVisibility>::CreateSP(this, &FToolkitBuilder::GetActivePaletteCommandsVisibility));

	ToolPaletteWidget->AddSlot()
		.HAlign(HAlign_Fill)
		.AutoHeight()
		[
			PaletteButtonsWidget.ToSharedRef()
		];
}

TSharedRef<SWidget> FToolkitBuilder::GetToolPaletteWidget() const
{
	return ToolPaletteWidget->AsShared();
}

TSharedRef<SWidget> FToolkitBuilder::GetContextMenuContent(const FName CommandName)
{
	FMenuBuilder MenuBuilder(true, nullptr);

	for (const TSharedRef<FEditablePalette>& EditablePalette : EditablePalettesArray)
	{
		const FUIAction ItemAction(FExecuteAction::CreateSP(
			SharedThis(this), 
			&FToolkitBuilder::ToggleCommandInPalette, 
			EditablePalette, CommandName.ToString()));
		const FText LoadPaletteActionCommandLabelFText = EditablePalette->LoadToolPaletteAction->GetLabel();
		const FText& ItemText = EditablePalette->IsInPalette(CommandName) ? 
			FText::Format(LOCTEXT("RemoveFromPalette", "Remove from {0}"), LoadPaletteActionCommandLabelFText) : 
			FText::Format(LOCTEXT("AddToPalette", "Add to {0}"), LoadPaletteActionCommandLabelFText);

		MenuBuilder.AddMenuEntry(ItemText, ItemText, FSlateIcon(), ItemAction);
		
	}

	return MenuBuilder.MakeWidget();
}

void FToolkitBuilder::ResetWidget()
{
	const bool bInitLoadToolPaletteMap = true;
	InitializeCategoryToolbar( bInitLoadToolPaletteMap );
	ToolPaletteWidget = SNew(SVerticalBox);
}

void FToolkitBuilder::ResetToolPaletteWidget()
{
	if (ToolPaletteWidget)
	{
		ToolPaletteWidget->ClearChildren();
		return;
	}
	ToolPaletteWidget = SNew(SVerticalBox);
}

bool FToolkitBuilder::HasSelectedToolSet() const
{
	return HasActivePalette();
}

void FToolkitBuilder::SetActivePaletteOnLoad(const FUICommandInfo* Command)
{
	if (const TSharedPtr<FToolPalette>* LoadPalette = LoadCommandNameToToolPaletteMap.Find(Command->GetCommandName().ToString()))
	{
		CreatePalette(*LoadPalette);
	}
}

void FToolkitBuilder::SetActiveToolDisplayName(FText InActiveToolDisplayName)
{
	ActiveToolDisplayName = InActiveToolDisplayName;
}

FText FToolkitBuilder::GetActiveToolDisplayName() const
{
	return ActiveToolDisplayName;
}

EVisibility FToolkitBuilder::GetActiveToolTitleVisibility() const
{
	return ActiveToolDisplayName.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible;
}

void FToolkitBuilder::UpdateContentForCategory( FName InActiveCategoryName, FText InActiveCategoryText )
{
	TSharedPtr<SHorizontalBox> ToolNameHeaderBox;
	
	if ( !MainContentVerticalBox.IsValid() )
	{
		return;
	}
	
	if (ToolkitSections->ModeWarningArea)
	{
		MainContentVerticalBox->AddSlot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.Padding(5)
		[
			ToolkitSections->ModeWarningArea->AsShared()
		];
	}

	if (ToolkitSections->Header)
	{
		MainContentVerticalBox->AddSlot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.Padding(0)
		[
			ToolkitSections->Header->AsShared()
		];
	}
	
	MainContentVerticalBox->AddSlot()
	.AutoHeight()
	.HAlign(HAlign_Fill)
	.Padding(0)
		[
			GetToolPaletteWidget()
		];

	MainContentVerticalBox->AddSlot()
		.AutoHeight()
		[ *FSeparatorTemplates::SmallHorizontalBackgroundNoBorder().BindVisibility(
			TAttribute<EVisibility>::CreateLambda([this] () { return ActivePaletteButtonVisibility; }))];
	
	MainContentVerticalBox->AddSlot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.Padding(0)
		[
			SNew(SBorder)
			.HAlign(HAlign_Fill)
			.Padding(Style.ActiveToolTitleBorderPadding)                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                                         
			.BorderImage(&Style.ToolDetailsBackgroundBrush)
			[
				SNew(SBorder)
				.Visibility(SharedThis(this), &FToolkitBuilder::GetActiveToolTitleVisibility)
				.BorderImage(&Style.TitleBackgroundBrush)
				.Padding(Style.ToolContextTextBlockPadding)
				[
					SAssignNew(ToolNameHeaderBox, SHorizontalBox)
					+SHorizontalBox::Slot()
					.Padding(0)
					.VAlign(EVerticalAlignment::VAlign_Center)
					.HAlign(EHorizontalAlignment::HAlign_Left)
					[
						SNew(STextBlock)
						.Justification(ETextJustify::Left)
						.Margin(0)
						.Font(Style.TitleFont)
						.Text(TAttribute<FText>(SharedThis(this), &FToolkitBuilder::GetActiveToolDisplayName))
						.ColorAndOpacity(Style.TitleForegroundColor)
					]
				]

			]
	];
	
	if (ToolkitSections->ToolPresetArea)
	{
		ToolNameHeaderBox->AddSlot()
		.HAlign(EHorizontalAlignment::HAlign_Right)
		[
			ToolkitSections->ToolPresetArea->AsShared()
		];
	}

	if (ToolkitSections->ToolWarningArea)
	{
		MainContentVerticalBox->AddSlot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.Padding(5)
		[
			ToolkitSections->ToolWarningArea->AsShared()
		];
	}

	if (ToolkitSections->ToolHeader)
	{
		MainContentVerticalBox->AddSlot()
		.AutoHeight()
		.HAlign(HAlign_Fill)
		.Padding(0)
		[
			ToolkitSections->ToolHeader->AsShared()
		];
	}

	if (ToolkitSections->DetailsView)
	{
		MainContentVerticalBox->AddSlot()
		.HAlign(HAlign_Fill)
		.FillHeight(1.f)
			[
			SNew(SBorder)
			.BorderImage(&Style.ToolDetailsBackgroundBrush)
			.Padding(0.f, 2.f, 0.f, 2.f)
				[
					ToolkitSections->DetailsView->AsShared()
				]
		
			];
	}
	if (ToolkitSections->Footer)
	{
		MainContentVerticalBox->AddSlot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Bottom)
			.Padding(0)
		[
			ToolkitSections->Footer->AsShared()
		];
	}
}

void FToolkitBuilder::SetActivePaletteCommandsVisibility(EVisibility Visibility)
{
	ActivePaletteButtonVisibility = Visibility;
}

EVisibility FToolkitBuilder::GetActivePaletteCommandsVisibility() const
{
	return ActivePaletteButtonVisibility;
}

#undef LOCTEXT_NAMESPACE