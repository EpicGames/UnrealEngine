// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolkitStyle.h"

#include "FToolkitWidgetStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Interfaces/IPluginManager.h"
#include "Styling/SlateStyle.h"
#include "Styling/StarshipCoreStyle.h"
#include "Styling/StyleColors.h"
#include "Styling/CoreStyle.h"

TSharedPtr< FSlateStyleSet > FToolkitStyle::StyleSet = nullptr;
FName FToolkitStyle::StyleName("ToolkitStyle");

FToolkitStyle::FToolkitStyle()
	: FSlateStyleSet(StyleName)
{
	
}

void FToolkitStyle::Initialize()
{
	if (!StyleSet.IsValid())
	{
		StyleSet = Create();
		FSlateStyleRegistry::RegisterSlateStyle(*StyleSet);
	}
	
	FToolkitWidgetStyle Style;

	Style.SetTitleBackgroundBrush(FSlateRoundedBoxBrush(FStyleColors::Header, 0.f, FLinearColor(0, 0, 0, .8f), 0))
		 .SetToolDetailsBackgroundBrush( FSlateRoundedBoxBrush(FStyleColors::Background, 0.f, FLinearColor(0, 0, 0, .8f), 0))
		 .SetTitleForegroundColor(FStyleColors::Foreground)
		 .SetTitlePadding(FMargin(16.f, 4.f))
	 	 .SetTitleFont(FCoreStyle::GetDefaultFontStyle("Bold", 10))
		 .SetActiveToolTitleBorderPadding(FMargin(0.f, 0.f, 0.f, 0.f))
		 .SetToolContextTextBlockPadding(FMargin(16.f, 4.f));

	StyleSet->Set("FToolkitWidgetStyle", Style);
}

void FToolkitStyle::Shutdown()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet);
	ensure(StyleSet.IsUnique());
	StyleSet.Reset();
}

const FName& FToolkitStyle::GetStyleSetName() const
{
	return StyleName;
}

TSharedRef< FSlateStyleSet > FToolkitStyle::Create()
{
	TSharedRef< FSlateStyleSet > Style = MakeShareable(new FSlateStyleSet(StyleName));

	return Style;
}

const ISlateStyle& FToolkitStyle::Get()
{
	return *StyleSet;
}


#undef LOCAL_IMAGE_BRUSH
#undef IMAGE_PLUGIN_BRUSH
