// Copyright Epic Games, Inc. All Rights Reserved.

#include "WidgetRegistrationModule.h"
#include "ToolkitStyle.h"
#include "Inputs/BuilderInputManager.h"
#include "Persistence/BuilderPersistenceManager.h"

IMPLEMENT_MODULE(FWidgetRegistrationModule, WidgetRegistration);

void FWidgetRegistrationModule::StartupModule()
{
	FToolkitStyle::Initialize();
	UBuilderPersistenceManager::Initialize();
	FBuilderInputManager::Initialize();
}
	
void FWidgetRegistrationModule::ShutdownModule()
{
	FToolkitStyle::Shutdown();
	UBuilderPersistenceManager::ShutDown();
	FBuilderInputManager::Shutdown();
}


