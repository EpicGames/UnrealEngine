// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"

namespace UE::DisplayBuilders
{
	class FBuilderKeys;
	
	/**
	* Provides identifiers for display builders and some methods for using them. These are also used as the keys into persistent storage for builders.
	*/
	class FBuilderKey
	{ 
	public:
		/**
		 * default constructor that initializes the key
		 */
		WIDGETREGISTRATION_API explicit FBuilderKey();
		
		/**
		* converts the BuilderKey to its FName
		*/
		WIDGETREGISTRATION_API FName ToName() const;

		/** 
		 * @return a combined key with Suffix
		 * 
		 * @param Suffix the FName that will provide the Suffix for the combined Key
		 */
		FString  GetKeyWithSuffix( const FName Suffix ) const;

		/**
		* @return true if this builder is set to None, meaning that it has not been initialized
		*/
		bool IsNone() const;
		
	private:

		friend FBuilderKeys;

		/**
		* The FName providing the identifier 
		*/
		const FName Key;
 		
		/**
		* The constructor which takes an FName which provides the identifier for this BuilderKey
		*
		* @param InKey the FName which provides the identifier for this BuilderKey
		*/
		FBuilderKey( const FName InKey ) :
			Key( InKey )	
		{
		}
	};

	/**
	 * FBuilderKeys provides keys registered for specific builders. 
	 */
	class FBuilderKeys
	{
	public:
	/**
	* Get the singleton FBuilderKeys 
	*/
		WIDGETREGISTRATION_API static const FBuilderKeys& Get();

		/** The FBuilderKey for the Place Actors FCategoryDrivenContentBuilder. This provides the key into the favorites storage for place actors. */
		WIDGETREGISTRATION_API const FBuilderKey& PlaceActors()  const;

		/** A "None" key to provide a null state when needed. */
		WIDGETREGISTRATION_API const FBuilderKey& None()  const;

	private:
		/**
	 * The constructor is kept private because these are for UE supported builders. 
	 */
		FBuilderKeys();
	};
}
