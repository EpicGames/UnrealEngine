// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "Textures/SlateIcon.h"

namespace UE::DisplayBuilders
{
	/**
	 * A class which provides a parameter object for builders consisting of an FText Label and a FSlateIcon icon
	 */
	class FLabelAndIconArgs
	{
	public:
		WIDGETREGISTRATION_API  explicit FLabelAndIconArgs( FText InLabel = FText::GetEmpty(), FSlateIcon InIcon = FSlateIcon() );
		
		FText Label;
		FSlateIcon Icon;
	};
}