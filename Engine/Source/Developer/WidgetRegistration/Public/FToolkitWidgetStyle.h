// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/StyleColors.h"
#include "Brushes/SlateNoResource.h"

#include "Styling/SlateWidgetStyle.h"
#include "Fonts/SlateFontInfo.h"
#include "FToolkitWidgetStyle.generated.h"

#define UE_API WIDGETREGISTRATION_API

/**
 * FToolkitWidgetStyle is the FSlateWidgetStyle that defines the styling of a ToolkitWidget 
 */
USTRUCT(BlueprintType)
struct FToolkitWidgetStyle : public FSlateWidgetStyle
{
	GENERATED_BODY()

	FToolkitWidgetStyle() :
		TitleBackgroundBrush(FSlateNoResource()),
		ToolDetailsBackgroundBrush(FSlateNoResource()),
		TitleForegroundColor(FStyleColors::Panel)
	{
		
	}

	static UE_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; }

	static UE_API const FToolkitWidgetStyle& GetDefault();
		UE_API virtual void GetResources(TArray<const FSlateBrush*>& OutBrushes) const override;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush TitleBackgroundBrush;
	UE_API FToolkitWidgetStyle& SetTitleBackgroundBrush(const FSlateBrush& InPaletteTitleBackgroundBrush);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush ToolDetailsBackgroundBrush;
	UE_API FToolkitWidgetStyle& SetToolDetailsBackgroundBrush(const FSlateBrush& InToolDetailsBackgroundBrush);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateColor TitleForegroundColor;
	UE_API FToolkitWidgetStyle& SetTitleForegroundColor(const FSlateColor& InTitleForegroundColor);
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FMargin TitlePadding;
	UE_API FToolkitWidgetStyle& SetTitlePadding(const FMargin& InTitlePadding);
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FMargin ActiveToolTitleBorderPadding;
	UE_API FToolkitWidgetStyle& SetActiveToolTitleBorderPadding(const FMargin& InActiveToolTitleBorderPadding);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FMargin ToolContextTextBlockPadding;
	UE_API FToolkitWidgetStyle& SetToolContextTextBlockPadding(const FMargin& InToolContextTextBlockPadding);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateFontInfo TitleFont;
	UE_API FToolkitWidgetStyle& SetTitleFont(const FSlateFontInfo& InTitleFont);
};

#undef UE_API
