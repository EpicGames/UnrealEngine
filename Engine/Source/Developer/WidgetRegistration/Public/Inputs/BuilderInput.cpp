// Copyright Epic Games, Inc. All Rights Reserved.

#include "Inputs/BuilderInput.h"
#include "Inputs/BuilderCommandCreationManager.h"

UE::DisplayBuilders::FBuilderInput UE::DisplayBuilders::FBuilderInput::NullInput;

UE::DisplayBuilders::FBuilderInput::~FBuilderInput()
{
	if ( !IsNameNone() && FBuilderCommandCreationManager::IsRegistered() )
	{
		FBuilderCommandCreationManager::Get().UnregisterCommandForBuilder( *this );		
	}
}

bool UE::DisplayBuilders::FBuilderInput::IsNameNone() const
{
	return Name.IsNone();
}

void UE::DisplayBuilders::FBuilderInput::InitializeCommandInfo()
{
	if ( !IsNameNone() )
	{
		FBuilderCommandCreationManager::Get().RegisterCommandForBuilder( *this );		
	}
}

UE::DisplayBuilders::FBuilderInput::FBuilderInput(
	FName InName,
	FText InLabel,
	FSlateIcon InIcon,
	EUserInterfaceActionType InUserInterfaceType,
	FText InToolTip,
	FText InDescription,
	TArray<TSharedRef<FInputChord>> InActiveChords ,
	FInputChord InDefaultChords ,
	FName InUiStyle ,
	FName InBindingContext  ,
	FName InBundle )
	: FLabelAndIconArgs( InLabel.IsEmpty() ? FText::FromName( InName ) : InLabel, InIcon )
	, Name( InName )
	, UserInterfaceType( InUserInterfaceType )
	, Description( InDescription.IsEmpty() ? Label : InDescription )
	, ActiveChords( InActiveChords )
	, DefaultChords( InDefaultChords )
	, UIStyle( InUiStyle )
	, BindingContext( InBindingContext )
	, Bundle( InBundle )
	, Index( INDEX_NONE )
	, Tooltip( InToolTip.IsEmpty() ? InLabel : InToolTip )
{
	InitializeCommandInfo();
}