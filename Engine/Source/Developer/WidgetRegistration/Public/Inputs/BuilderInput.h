// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Common/BuilderTypes.h"
#include "Templates/SharedPointer.h"
#include "Internationalization/Text.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Framework/MultiBox/SToolBarButtonBlock.h"
#include "Misc/CoreMiscDefines.h"

class FBuilderInputManager;
struct FInputChord;

namespace UE::DisplayBuilders
{
	/**
	 * BuilderInput is the input representation for Builders and provides conversions to the various Slate types for inputs.
	 */
	class FBuilderInput : public FLabelAndIconArgs
	{
	public:
		
		WIDGETREGISTRATION_API explicit FBuilderInput(
			FName InName = FName(),
			FText InLabel = FText::GetEmpty(),
			FSlateIcon InIcon = FSlateIcon(),
			EUserInterfaceActionType InUserInterfaceType = EUserInterfaceActionType::Button,
			FText InToolTip = FText::GetEmpty(),
			FText InDescription = FText::GetEmpty(),
			TArray<TSharedRef<FInputChord>> InActiveChords = {},
			FInputChord InDefaultChords = FInputChord(),
			FName InUiStyle = NAME_None,
			FName InBindingContext = NAME_None,
			FName InBundle = NAME_None );

		/**
		 * The destructor, destroys any command related information
		 */
		WIDGETREGISTRATION_API ~FBuilderInput();

		/** Name of the Input */
		FName Name;

		/** The type of user interface to associated with this action */
		EUserInterfaceActionType UserInterfaceType;

		/** Localized help text for the UI command */
		FText Description;
		
		/** Input commands that executes this action */
		TArray<TSharedRef<FInputChord>> ActiveChords;
		
		/** The default input chords for the UI command (can be invalid) */
		FInputChord DefaultChords;

		/** Brush name for icon to use in tool bars and menu items to represent the UI command in its toggled on (checked) state*/
		FName UIStyle;

		/** The context in which the UI command is active */
		FName BindingContext;

		/** The bundle to group the UI command into. The bundle must have been added to the BindingContext first. */
		FName Bundle;

		/** the index of the BuilderInput in whatever container it is in */
		int32 Index;
		
		/** the tooltip for the input */
		FText Tooltip;

		/** FUICommandInfo, the basic Slate Class for defining user-facing commands, including their appearance and behavior in the UI  */
		TSharedPtr<FUICommandInfo> UICommandInfo;

		/** FButtonArgs, a type which can be used to create buttons with FToolbarBuilder */
		FButtonArgs ButtonArgs;
		
		/** For use when we need a null state */
		static FBuilderInput NullInput;
		
		/**
		 * @return true if the input name is equivalent to Name_NONE
		 */
		bool IsNameNone() const;

	private:
		/**
		 * Initializes the related FUICommandInfo
		 */
		void InitializeCommandInfo();
	};
}
