// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ToolkitBuilder.h"
#include "ToolkitStyle.h"
#include "Layout/CategoryDrivenContentBuilderBase.h"
#include "Inputs/BuilderInput.h"
#include "Misc/CoreMiscDefines.h"

class FColumnWrappingContainer;
class FSimpleTitleContainer;
class FSlateBuilder;

namespace UE::DisplayBuilders
{
	class FBuilderInput;
}
	
/**
 * A builder which creates a widget that has a vertical toolbar category picker on the left
 * hand side which populates the content on the right side.
 */
class  FCategoryDrivenContentBuilder : public FCategoryDrivenContentBuilderBase
{
public:
	
	
	DECLARE_DELEGATE_RetVal_OneParam( TSharedRef<SWidget>, FProvideSelectedCategoryContent, const FName&)
	FProvideSelectedCategoryContent  ProvideSelectedCategoryContentDelegate;

	DECLARE_DELEGATE_TwoParams( FUpdateContentForCategoryDelegate, FName, FText);

	/**
	 * @return the TArray<FName> of Favorites for this 
	 */
	WIDGETREGISTRATION_API const TArray<FName>& GetFavorites() const;

	/**
	 * @return the context menu containing an item to add or remove a favorite. If FavoritesItemName is a
	 * current favorite it will return a context menu with the proper item to add or remove the favorite
	 * 
	 * @param FavoritesItemName the name of the Favorites item that would be toggled with the menu
	 */
	WIDGETREGISTRATION_API TSharedRef<SWidget> CreateFavoritesContextMenu( FString FavoritesItemName );

	/**
	 * Returns the context menu containing an item to show or hide category labels.
	 * 
	 * @param bInShowCategoryLabelsItemName the name of the Show Category Labels item that would be toggled with the menu
	 */
	WIDGETREGISTRATION_API TSharedRef<SWidget> CreateShowCategoryLabelsContextMenu();

	/**
	 * A delegate which provides the const FName& Command name as a parameter to indicates which category was clicked.
	 * The bound method should update this FCategoryDrivenContentBuilder's content based on the category that was chosen.
	 */ 
	FUpdateContentForCategoryDelegate UpdateContentForCategoryDelegate;
	
public:
	/**
	 * initializes this FCategoryDrivenContentBuilder with the given FCategoryDrivenContentBuilderArgs
	 * @param Args the parameter object which provides the information to initialize this FCategoryDrivenContentBuilder 
	 */
	WIDGETREGISTRATION_API explicit FCategoryDrivenContentBuilder(FCategoryDrivenContentBuilderArgs& Args);
	
	/**
	 * destroys the FCategoryDrivenContentBuilder and frees any resources
	 */
	WIDGETREGISTRATION_API ~FCategoryDrivenContentBuilder();

	/*
	 * Initializes the data necessary to build the category toolbar
	 */
	WIDGETREGISTRATION_API virtual void InitializeCategoryToolbar() override;

	/**
	 * Generate the widget built by this builder and return a shared pointer to it.
	 */
	virtual TSharedPtr<SWidget> GenerateWidget() override;

	/*
	 * Refreshes the existing widget.
	 */
	WIDGETREGISTRATION_API virtual void UpdateWidget() override;


	/**
	 * Sets whether we should show that no category selected. If true, the category picker shows no selection and there is no category title.
	 *
	 * @param bInShowNoCategorySelection if true, the category picker will show no selection and the category title is not shown
	 */
	WIDGETREGISTRATION_API void SetShowNoCategorySelection( bool bInShowNoCategorySelection );
	
	/*
	 * Initializes the category buttons with the given FBuilderInputs
	 *
	 * @param InBuilderInputArray the array of FBuilderInput instances that will initialize the Category buttons
	 */
	WIDGETREGISTRATION_API void InitializeCategoryButtons(TArray<UE::DisplayBuilders::FBuilderInput> InBuilderInputArray);

	/*
	 * Initializes the category buttons with the given FBuilderInputs
	 */
	WIDGETREGISTRATION_API void InitializeCategoryButtons();

	/**
	 * Converts the SWidget Widget to a FSlateBuilder and adds it to the main content for the currently selected category
	 *
	 * @param Widget the SWidget to convert to an FSlateBuilder and add to the content for the current category
	 */
	WIDGETREGISTRATION_API void AddBuilder( TSharedRef<SWidget> Widget );

	/**
	 * Converts the SWidget Widget to a FSlateBuilder and sets it as the entire main content for the currently selected category
	 *
	 * @param Widget the SWidget to convert to an FSlateBuilder and set as the content for the current category
	 */
	WIDGETREGISTRATION_API void FillWithBuilder( TSharedRef<SWidget> Widget );

	/**
	 * Clears the content for the currently selected category
	 */
	WIDGETREGISTRATION_API void ClearCategoryContent();
	
	/**
	 * Adds the favorites with the name InFavoriteCommandName to the favorites list. If InFavoriteCommandName is already in
	 * the Favorites list, this is a no-op
	 */
	WIDGETREGISTRATION_API void AddFavorite( FName InFavoriteCommandName );
	
private:
	/**
     * Given the active category name, return an SWidget to provide the content
     * 
     * @param ActiveCategoryName the name of the Category that will be loaded, which will be the title in the header of the returned content
     */
	virtual void UpdateContentForCategory(
		FName ActiveCategoryName = NAME_None,
		FText InActiveCategoryText = FText::GetEmpty() ) override;

	/**
	* A map of the Category FName to the TSharedRef<FBuilderInput> providing the information to instantiate the category buttons
	*/
	TMap<FName, UE::DisplayBuilders::FBuilderInput> CategoryNameToBuilderInputMap;

	/**
	 * The TArray of FNames which have been set as favorites
	 */
	TArray<FName> Favorites;

	/**
	 * the FName of the favorites Category
	 */
	FName FavoritesCategoryName;

	/**
	 * Toggles the favorite with the name InFavoriteCommandName
	 */
	void ToggleFavorite( FName InFavoriteCommandName );

	/**
	 * Toggles the visibility of the categories names
	 */
	void ToggleShowLabels();

private:

	/** The array of BuilderInputs, in the correct display order */
	TArray<UE::DisplayBuilders::FBuilderInput> BuilderInputArray;

	/** The array of FSlateBuilders that will build the content for the currently selected category */
	TArray<TSharedRef<FSlateBuilder>> ChildBuilderArray;

	/** the FText label of the currently selected category */
	FText CategoryLabel;
	
	/** the simple title container that has a header with the currently selected category name, and an empty body that will contain
	 * the category content */
	TSharedPtr<FSimpleTitleContainer> TitleContainer;

	/** the column wrapping container that provides the body of the category content */
	TSharedPtr<FColumnWrappingContainer> ColumnWrappingContainer;

	/** if true, the content for the category should have a single widget fill the content */
	bool bIsFilledWithWidget;

	/** if true, we need to show no selection on the category picker and no category title, else we should show both  */
	bool bShowNoCategorySelection;
};
