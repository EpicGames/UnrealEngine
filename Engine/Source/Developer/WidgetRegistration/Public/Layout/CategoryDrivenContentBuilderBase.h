// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "FToolkitWidgetStyle.h"
#include "ToolElementRegistry.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SSplitter.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "BuilderKey.h"
#include "Framework/MultiBox/SToolBarButtonBlock.h"

class SWidget;
class FToolElement;

class FCategoryDrivenContentBuilderArgs;

/**
 * The FToolElementRegistrationArgs which is specified for Toolkits
 */
class FCategoryDrivenContentBuilderBase : public FToolElementRegistrationArgs
{
public:

	// Used to specify what happens when you click the category button of a category that is already active.
	enum class ECategoryReclickBehavior : uint8
	{
		// Do nothing if the same category button is clicked.
		NoEffect,

		// Toggle the active category off, so no category is active.
		ToggleOff,

		// Do the same thing that would be done if we were switching from a different category. Note that
		// this will trigger OnActivePaletteChanged.
		TreatAsChanged,
	};

	/**
	 * A constructor for the FCategoryDrivenContentBuilderBase which defines the builder name
	 * @param InBuilderName the name for the builder
	 */
	explicit  FCategoryDrivenContentBuilderBase( FName InBuilderName = "FCategoryDrivenContentBuilderBase" );

	/**
	 * A constructor for the FCategoryDrivenContentBuilderBase which takes a
 	* FCategoryDrivenContentBuilderArgs to initialize this
 	* @param Args the FCategoryDrivenContentBuilderArgs used to initialize this FCategoryDrivenContentBuilderBase
 	*/
	explicit FCategoryDrivenContentBuilderBase( FCategoryDrivenContentBuilderArgs& Args );

	virtual  ~FCategoryDrivenContentBuilderBase() override;

	/*
	 * Initializes the data necessary to build the category toolbar
	 */
	virtual void InitializeCategoryToolbar() = 0;

	/**
	 * Initializes the category toolbar container VBox, and the children inside it.
	 * On any repeat calls, the SVerticalBox created on the first pass will be
	 * emptied and the children repopulated.
	 */
	void InitCategoryToolbarContainerWidget();

	/**
	 * Sets category button label visibility to Visibility. It also reinitializes the
	 * category toolbar data, as the toolbar's LabelVisibility member is now stale.
	 *
	 * @param Visibility If Visibility == EVisibility::Collapsed, the category button labels
	 * will be shown, else they will not be shown.
	 */
	WIDGETREGISTRATION_API void SetCategoryButtonLabelVisibility(EVisibility Visibility);

	/**
	 * Sets category button label visibility to Visibility. It also reinitializes the
	 * category toolbar data, as the toolbar's LabelVisibility member is now stale.
	 *
	 * @param bIsCategoryButtonLabelVisible If bIsCategoryButtonLabelVisible == true,
	 * the category button labels will be shown, else they will not be shown.
	 */
	WIDGETREGISTRATION_API void SetCategoryButtonLabelVisibility(bool bIsCategoryButtonLabelVisible);

	/**
	 * refreshes the UI display of the category toolbar
	 */
	WIDGETREGISTRATION_API void RefreshCategoryToolbarWidget( bool bShouldReinitialize = false );

	/** Implements the generation of the TSharedPtr<SWidget> */
	WIDGETREGISTRATION_API virtual TSharedPtr<SWidget> GenerateWidget() override;

	/** Creates the Toolbar for the widget with the FUICommandInfos that load the Palettes */
	TSharedRef<SWidget> CreateToolbarWidget() const;

	/** @return a TSharedPointer to the FToolbarBuilder with the FUICommandInfos that load the Palettes */
	TSharedPtr<FToolBarBuilder> GetLoadPaletteToolbar();
	
	/** a TSharedPointer to the FToolElement for the vertical toolbar */
	TSharedPtr<FToolElement> VerticalToolbarElement;

	/**   OnActivePaletteChanged is broadcast when the active palette changes to a different palette  */
	FSimpleMulticastDelegate OnActivePaletteChanged;

	/**  Delegate that takes the default toolbar button created by this as a parameter, adds any decorator needed and returns the new widget*/
	FGetDecoratedButtonDelegate GetDecoratedButtonDelegate;
	
private:

	/* The SWidget that is the whole Toolkit */
	TSharedPtr<SWidget> ToolkitWidget;


	/** The SVerticalBox which contains the category toolbar */
	TSharedPtr<SVerticalBox> CategoryToolbarVBox;

	/** The SVerticalBox which holds the entire Toolkit */
	TSharedPtr<SVerticalBox> ToolkitWidgetContainerVBox;

protected:

	/**
	 * Creates the SWidget
	 */
	void CreateWidget();

	FName GetCategoryToolBarStyleName() const;
	
	virtual void UpdateContentForCategory( FName ActiveCategoryName = NAME_None, FText InActiveCategoryText = FText::GetEmpty() ) = 0;
	
	/** The SVerticalBox which holds the main content ~ -all but the Category chooser */
	TSharedPtr<SVerticalBox> MainContentVerticalBox;
	
	/** the tool element registry this class will use to register UI tool elements */
	static FToolElementRegistry ToolRegistry;
	
	/** Specifies what happens if you click the category button of an already-active category */
	ECategoryReclickBehavior CategoryReclickBehavior = ECategoryReclickBehavior::NoEffect;
	
	/** The current FToolkitWidgetStyle */
	FToolkitWidgetStyle Style;
	
	/** If CategoryButtonLabelVisibility == EVisibility::Visible, the category button
	 * labels are visible, else they are not displayed. By default the selected category button labels are Visible    */	
	EVisibility CategoryButtonLabelVisibility;
	
	/*
	 * Is the Category toolbar visible?
	 */
	EVisibility CategoryToolbarVisibility = EVisibility::Visible;
	
	/** A TSharedPointer to the FUICommandList for the FUICommandInfos which load a tool palette */
	TSharedPtr<FUICommandList> LoadToolPaletteCommandList;
	
	/** The toolbar builder for the toolbar which has the FUICommandInfos which load the various palettes */
	TSharedPtr<FToolBarBuilder> LoadPaletteToolBarBuilder;

	/** Name of this builder */
	FName BuilderName;
	
	/**
	 * the name of the currently selected/active category
	 */
	FName ActiveCategoryName;
};

/**
 * A struct to provide arguments for a FCategoryDrivenContentBuilderBase
 */
class FCategoryDrivenContentBuilderArgs
{
public:
	/**
	* Constructor taking the builder name and the FBuilderKey
	*
	* @param InBuilderName the name of the builder
	* @param InKey the FBuilderKey for this 
	*/
	WIDGETREGISTRATION_API FCategoryDrivenContentBuilderArgs(
		FName InBuilderName, const UE::DisplayBuilders:: FBuilderKey& InKey = UE::DisplayBuilders::FBuilderKeys::Get().None() );

	/** the FBuilderKey for this */
	UE::DisplayBuilders::FBuilderKey Key;

	/** Name of this builder */
	FName BuilderName;
	
	/** If bShowCategoryButtonLabels == true, the category button
   * labels should be visible, else they are not displayed  */
	bool bShowCategoryButtonLabels;

	/** Specifies what happens if you click the category button of an already-active category */
	FCategoryDrivenContentBuilderBase::ECategoryReclickBehavior CategoryReclickBehavior;

	/** The FName of the favorites Category, if one exists */
	FName FavoritesCommandName;

	/** The label/title of the initially selected category  */
	FText CategoryLabel;
	
	/** The name of the initially selected category */
	FName ActiveCategoryName;
	
	/**  Delegate that takes the default toolbar button created by this as a parameter, adds any decorator needed and returns the new widget */
	FGetDecoratedButtonDelegate GetDecoratedButtonDelegate;
};