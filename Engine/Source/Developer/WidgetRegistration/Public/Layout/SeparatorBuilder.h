// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ToolElementRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styles/SlateBrushTemplates.h"
#include "Styling/StyleColors.h"
#include "Widgets/Layout/SSeparator.h"

/**
 * Provides a type that will hold the size for a FSeparatorBuilder
 */
class FSeparatorSize
{
	/**
	 * The magnitude of the size ~ the greater the Magnitude the larger the Separator.
	 */
	float Magnitude;

	/**
	 * Constructs the Separator Size with a Magnitude o InMagnitude
	 *
	 * @param InMagnitude 
	 */
	FSeparatorSize(float InMagnitude): Magnitude(InMagnitude)
	{
	}
	
public:

	/**
	 * Converts the Size to a float value
	 */
	float ToFloat() const
	{
		return Magnitude;
	}

	friend struct FSeparatorSizes;
};

/**
 * A struct to contain the possible sizes for a Separator 
 */
struct FSeparatorSizes
{
	/** the size of an extra small (1 slate unit) Separator */
	static inline const FSeparatorSize XSmall{1.0f};
	
	/** the size of a small (2 slate unit) Separator */
	static inline const FSeparatorSize Small{2.0f};
	
	/** the size of a medium (4 slate unit) Separator */
	static inline const FSeparatorSize Medium{4.0f};
	
	/** the size of a large (8 slate unit) Separator */
	static inline const FSeparatorSize Large{8.0f};
	
	/** the size of an extra large (16 slate unit) Separator */
	static inline const FSeparatorSize XLarge{16.0f};
};

/**
 * FSeparatorBuilder instances will build a TSharedRef<SSeparator>
 */
class FSeparatorBuilder final : public FToolElementRegistrationArgs
{
public:

	/**
	 * Implements the generation of the TSharedPtr<SWidget>
	 */
	virtual TSharedPtr<SWidget> GenerateWidget() override;

	/**
	 * Generates the SSeparator and returns the TSharedRef<SSeparator> as a reference to it
	 *
	 * @return the TSharedRef<SSeparator> as a reference to the generated SSeparator
	 */
	TSharedRef<SSeparator> ToSSeparatorSharedRef();

	/**
	 * Generates the SSeparator and returns the TSharedRef<SSeparator> as a reference to it.
	 * This provides some convenience syntax to generate the widget and get a reference to it
	 * in a concise way.
	 *
 	 * @return the TSharedRef<SSeparator> as a reference to the generated SSeparator
	 */
	WIDGETREGISTRATION_API TSharedRef<SSeparator> operator*();

	/**
	 * Initializes the size of the separator. If called after the widget creation (that is, a call to GenerateWidget)
	 * this call will have no effect.
	 *
	 * @param NewSize the FSeparatorSize& that will provide the size for the Separator
	 * @return this FSeparatorBuilder to enable cascading.
	 */
	WIDGETREGISTRATION_API FSeparatorBuilder& InitializeSize(const FSeparatorSize& NewSize);

	/** 
	 * Sets the color of the Separator. This will update the color of the Separator even after widget creation.
	 *
	 * @param NewColor the EStyleColor to set the color of the SSeparator. Note that the EStyleColor is a key into
	 * the color table, so providing an EStyleColor such as EStyleColor::Panel will honor updates to changes in themes
	 * for that EStyleColor immediately.
	 * @return this FSeparatorBuilder to enable cascading.
	 */
	WIDGETREGISTRATION_API FSeparatorBuilder& SetColor(const EStyleColor& NewColor);

	/** 
	 * Binds the Visibility of the Separator to a TAttribute for continuous update.
	 *
	 * @param NewVisibility the TAttribute<EVisibility> to bind for the SSeparator. 
	 * @return this FSeparatorBuilder to enable cascading.
	 */
	FSeparatorBuilder& BindVisibility(TAttribute<EVisibility> NewVisibility)
	{
		Visibility = NewVisibility;
		return *this;
	}
	
private:

	friend struct FSeparatorTemplates;

	/**
	 * Constructs the FSeparatorBuilder. This is private to keep the end user API simple. By starting with a template
	 * callers should not have a need to construct an FSeparatorBuilder.
	 *
	 * @param InColor the EStyleColor to provide the color for this Separator
	 * @param InOrientation the EOrientation that sets the orientation for this Separator
	 * @param InSize the FSeparatorSize that sets the size for this Separator
	 * @param InImage the const FSlateBrush* which specifies the FSlateBrush to use for this Separator 
	 */
	FSeparatorBuilder(EStyleColor InColor = EStyleColor::Background,
		              EOrientation InOrientation = Orient_Horizontal,
					  FSeparatorSize InSize = FSeparatorSizes::Small,
					  const FSlateBrush* InImage = FAppStyle::GetBrush("ThinLine.Horizontal"));

	/**
	 * The const FSlateBrush* for the SSeparator
	 */
	const FSlateBrush* Image = FAppStyle::GetBrush("ThinLine.Horizontal");

	/**
	 * The EOrientation that specifies whether this is a horizontal or vertical Separator
	 */
	EOrientation Orientation = Orient_Horizontal;

	/**
	 * The FSlateColor that specifies the Color for this Separator. 
	 */
	TAttribute<FSlateColor> SlateColor = EStyleColor::Background;

	/**
	 * The FSeparatorSize that specifies the Size for this Separator.
	 */
	float Size = FSeparatorSizes::Small.ToFloat();

	/**
	 * The TAttribute<EVisibility> which will provide the Visibility for the Separator
	 */
	TAttribute<EVisibility> Visibility = EVisibility::Visible;

};
