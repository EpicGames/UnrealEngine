// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "SeparatorBuilder.h"


/**
 * A struct which holds methods that provide templates for FSeparatorBuilders. The provided FSeparatorBuilders
 * can be updated as needed with the FSeparatorBuilder API.
 */
struct FSeparatorTemplates
{
	/**
	* Creates and returns a small (2 Slate Unit), vertical FSeparatorBuilder with the color of the theme
	 * specified color for Background
	 *
	 * @return a small (2 Slate Unit), vertical FSeparatorBuilder with the color of the theme
	 * specified color for Background
	 */
	WIDGETREGISTRATION_API static FSeparatorBuilder SmallVerticalBackgroundNoBorder();

	/**
	* Creates and returns a small (2 Slate Unit), horizontal FSeparatorBuilder set to the color of the theme
	* specified color for Background
	*
	* @return a small (2 Slate Unit), horizontal FSeparatorBuilder with the color of the theme
	* specified color for Background
	 */
	WIDGETREGISTRATION_API static FSeparatorBuilder SmallHorizontalBackgroundNoBorder();

	/**
	* Creates and returns a small (2 Slate Unit), horizontal FSeparatorBuilder set to the color of the theme
	 * specified color for Panels.
	 *
	 * @return a small (2 Slate Unit), horizontal FSeparatorBuilder with the color of the theme
	 * specified color for Panels.
	 */
	WIDGETREGISTRATION_API static FSeparatorBuilder SmallHorizontalPanelNoBorder();
};
