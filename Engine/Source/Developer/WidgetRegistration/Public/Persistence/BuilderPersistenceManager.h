// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "EditorConfigBase.h"
#include "BuilderKey.h"
#include "UObject/ObjectPtr.h"

#include "BuilderPersistenceManager.generated.h"

#define UE_API WIDGETREGISTRATION_API

/**
 * Manages an array of FNames to persist
 */
USTRUCT()
struct FPersistedNameArray
{
	GENERATED_BODY()
public:
	
	UPROPERTY()
	TArray<FName> ArrayOfNamesToPersist;
};

/**
 * Manages a Bool to persist
 */
USTRUCT()
struct FPersistedBool
{
	GENERATED_BODY()
public:
	
	UPROPERTY()
	bool PersistedBool = false;
};


/**
 * The Builder Persistence Manager handles persistence for Builders through use of FBuilderKeys
 */
UCLASS(MinimalAPI, EditorConfig="BuilderPersistenceManager")
class  UBuilderPersistenceManager : public UEditorConfigBase
{
	GENERATED_BODY()

public:
	/**
	 * @return the default named favorites array for the Builder with FBuilderKey Key, if one exists, else an empty array is returned
	 * 
	 * @param Key the FBuilderKey to retrieve the favorites for
	 */
	UE_API TArray<FName> GetFavoritesNames( const UE::DisplayBuilders::FBuilderKey& Key );

	/**
	 * Sets the default named favorites array for the Builder with FBuilderKey Key
	 * 
	 * @param Key the FBuilderKey to retrieve the favorites for
	 * @param Favorites the array of FNames for the favorites to persist
	 */
	UE_API void PersistFavoritesNames( const UE::DisplayBuilders::FBuilderKey& Key, TArray<FName>& Favorites );

	/**
	 * Sets the default button label EVisisbility for the Builder with FBuilderKey Key
	 * 
	 * @param Key the FBuilderKey to retrieve the button label Bool for
	 * @param Bool the bool for the button labels to persist
	 */
	UE_API void PersistShowButtonLabels( const UE::DisplayBuilders::FBuilderKey& Key, bool bValue );
	
	/**
	 * @return the bool or the buttons labels for the Builder with FBuilderKey Key, if one exists, else an empty array is returned
	 * 
	 * @param Key the FBuilderKey to retrieve the button label bool
	 * @param PersistedBoolIfNoneFound the Bool to set as the new persistence value, if one is not found
	 */
	UE_API bool GetShowButtonLabels( const UE::DisplayBuilders::FBuilderKey& Key, bool bDefaultValue );

	/**
	 * Initialize the Persistence manager
	 */
	static UE_API void Initialize();
	
	/**
	* Shuts down the Persistence manager
	*/
	static UE_API void ShutDown();

	/**
	 * Gets the singleton for the Builder Persistence Manager
	 */
	static UBuilderPersistenceManager* Get()
	{
		return Instance;
	}


private:

	UPROPERTY(meta=(EditorConfig))
	TMap<FString, FPersistedNameArray> SavedNameToPersistedFNameArrayMap;

	UPROPERTY(meta=(EditorConfig))
	TMap<FString, FPersistedBool> SavedNameToPersistedBoolMap;
	
	static UE_API TObjectPtr<UBuilderPersistenceManager> Instance;

private:
	/**
	 * @return the an array for the Builder with FBuilderKey Key and the suffix that the array was persisted with, if one exists, else an empty array is returned
	 * 
	 * @param Key the FBuilderKey to retrieve the FNames array for
	 * @param PersistenceKeySuffix the suffix to add to the FBuilderKey to persist the array of FNames with
	 */
	UE_API TArray<FName> GetPersistedArrayOfNames( const UE::DisplayBuilders::FBuilderKey& Key, FName PersistenceKeySuffix );

	/**
	 * Persists the array of FNames the Builder with FBuilderKey Key and suffix PersistenceKeySuffix
	 * 
	 * @param Key the FBuilderKey to retrieve the favorites for
	 * @param PersistenceKeySuffix the suffix added to the FBuilderKey to persist the array of FNames with
	 */
	UE_API void PersistArrayOfNames( const UE::DisplayBuilders::FBuilderKey& Key, FName PersistenceKeySuffix, TArray<FName>& ArrayOfNamesToPersist );

	/**
	 * Returns the bool for the Builder with FBuilderKey Key and the suffix that the array was persisted with, if one exists, else an empty array is returned
	 * 
	 * @param Key the FBuilderKey to retrieve the bool
	 * @param PersistenceKeySuffix the suffix to add to the FBuilderKey to get the bool with
	 * @param bDefaultValue the Bool to set as the new persistence value, if one is not found
	 */
	UE_API bool GetPersistedBool( const UE::DisplayBuilders::FBuilderKey& Key, FName PersistenceKeySuffix, bool bDefaultValue );
	
	/**
	 * Persists the Bool the Builder with FBuilderKey Key and suffix PersistenceKeySuffix
	 * 
	 * @param Key the FBuilderKey to retrieve the favorites for
	 * @param PersistenceKeySuffix the suffix added to the FBuilderKey to persist the Bool with
	 */
	UE_API void PersistBool( const UE::DisplayBuilders::FBuilderKey& Key, FName PersistenceKeySuffix, bool bInPersistedBool );
};

#undef UE_API
