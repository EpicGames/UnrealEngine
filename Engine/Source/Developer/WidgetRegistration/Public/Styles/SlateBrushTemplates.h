// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Brushes/SlateColorBrush.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateBrush.h"
#include "Styling/StyleColors.h"

#define UE_API WIDGETREGISTRATION_API

/** A const FSlateBrush* Factory */
struct UE_DEPRECATED(5.8, "This struct is deprecated. You should use FAppStyle instead") FSlateBrushTemplates
{
	// Images
	// UE_DEPRECATED 5.8 : Use FAppStyle::GetBrush("VerticalBoxDragIndicatorShort") instead
	static UE_API const FSlateBrush* DragHandle();
	// UE_DEPRECATED 5.8 : "Use FAppStyle::GetBrush("ThinLine.Horizontal") instead
	static UE_API const FSlateBrush* ThinLineHorizontal();

	// Colors
	// UE_DEPRECATED 5.8 : Use FAppStyle::GetBrush("Brushes.Dropdown") instead
	static UE_API const FSlateBrush* Dropdown();
	// UE_DEPRECATED 5.8 : Use FAppStyle::GetBrush("Brushes.Transparent") instead
	static UE_API const FSlateBrush* Transparent();
	// UE_DEPRECATED 5.8 : Use FAppStyle::GetBrush("Brushes.Panel") instead
	static UE_API const FSlateBrush* Panel();
	// UE_DEPRECATED 5.8 : Use FAppStyle::GetBrush("Brushes.Recessed") instead
	static UE_API const FSlateBrush* Recessed();

	/**
	 * gets a const FSlateBrush* with the color Color
	 *
	 *  @param the EStyleColor we need the slate brush for
	 */
	// UE_DEPRECATED 5.8 : Use FAppStyle::GetBrush() with a predefined brush name in StyleSheets instead
	UE_API const FSlateBrush* GetBrushWithColor(EStyleColor Color);

	/**
	 * gets the FSlateBrushTemplates singleton
	 */
	//UE_DEPRECATED 5.8: This struct is deprecated. You should use FAppStyle instead
	static UE_API FSlateBrushTemplates& Get();

	/** the map of EStyleColor to const FSlateColorBrush UniquePtr */
	TMap<EStyleColor, TUniquePtr<FSlateColorBrush>> EStyleColorToSlateBrushMap;
};

#undef UE_API
