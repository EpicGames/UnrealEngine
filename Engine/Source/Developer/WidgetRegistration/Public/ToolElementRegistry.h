//  Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BuilderKey.h"
#include "Widgets/SWidget.h"

#define UE_API WIDGETREGISTRATION_API

class FToolElementRegistry;

/** An enum to provide the type of Tool Element*/
enum class EToolElement : uint32
{
	Toolbar = 0,
	Toolkit,
	Separator,
	Section
};

/** A class which provides the data necessary to generate a widget. This class should be
 * extended and used to hold whatever information is needed to generate or update the
 * widgets that make up the tool, and implement the virtual widget methods
 */
class FToolElementRegistrationArgs :
	public TSharedFromThis<FToolElementRegistrationArgs>
{
	
public:

	/** The constructor, which takes an EToolElement which will define the type of Tool Element*/
	UE_API FToolElementRegistrationArgs(EToolElement InToolElementType);

	/** The constructor, which takes an EToolElement which will define the type of Tool Element*/
	UE_API FToolElementRegistrationArgs(FName InStyleClassName);
	
	/**
	 * The constructor, which takes an FBuilderKey which will define the type of Tool Element
	 *
	 * @param InBuilderKey the FBuilderKey that provides the Key for this builder
	 */
	UE_API FToolElementRegistrationArgs( UE::DisplayBuilders::FBuilderKey InBuilderKey );

	/** default destructor in case any subclasses need to provide a destructor*/
	virtual ~FToolElementRegistrationArgs() = default;

	/** Implements the generation of the TSharedPtr<SWidget> */
	UE_API virtual TSharedPtr<SWidget> GenerateWidget();

	/** Implements the generation of the TSharedRef<SWidget> */
	UE_API TSharedRef<SWidget> GenerateWidgetSharedRef();

	/** Updates/reloads this widget. This should be called after a consumer has changed any Data in this */
	UE_API virtual void UpdateWidget();

	/** Resets the widget to its initial state */
	UE_API virtual void ResetWidget();

	/** The type of tool element this is Registration args for */
	const EToolElement ToolElementType;

	/** The style class name */
	const FName StyleClassName;
	
	/** 
	* the FBuilderKey for this. This provides a key into persistence, if any exists for the builder, along 
	* with other things.
    */
	const UE::DisplayBuilders::FBuilderKey BuilderKey;
};

/** Serves as a key into the FToolElementRegistry of FToolElements */
class FToolElementRegistrationKey :
	public TSharedFromThis<FToolElementRegistrationKey> {

public:

	/** Constructor for FToolElementRegistrationKey.
	 *
	 * @param InName the FName that will be part of the compound key for the FToolElement this denotes
	 * @param InToolElementType the EToolElementType that will be part of the compound key for the FToolElement this denotes
	 */
	UE_API FToolElementRegistrationKey(FName InName, EToolElement InToolElementType );

	/** the FName that will be part of the compound key for the FToolElement this denotes */
	FName Name;

	/** the EToolElementType that will be part of the compound key for the FToolElement this denotes */ 
	EToolElement ToolElementType;

	UE_API FString GetKeyString();

private:
FString KeyString;
};

/** Represents one Tool Element, and is responsible for displaying the UI for that element  */
class FToolElement : public FToolElementRegistrationKey
{

public:

	friend class FToolElementRegistry;

	/** 
	 * FToolElement constructor
	 *
	 * @param InName the FName of the FToolElement
	 * @param InBuilderArgs the shared reference to the FToolElementRegistrationArgs. Need to have a shared reference
	 * here so that if the registered outlives the registrar, we maintain a copy
	 */
	UE_API FToolElement(
		const FName InName,
		TSharedRef<FToolElementRegistrationArgs> InBuilderArgs);

	/**
	 * a convenience pass through method to the FToolElementRegistrationArgs GenerateWidget method
	 * which generates the TSharedRef<SWidget>
	 *
	 * @return the TSharedRef<SWidget> which makes up the UI for this tool element
	 */
	UE_API TSharedRef<SWidget> GenerateWidget();

	/*
	 * Sets the FToolElementRegistrationArgs for the this tool element to RegistrationArgs
	 *
	 * @param RegistrationArgs the FToolElementRegistrationArgs for the this tool element
	 */
	UE_API void SetRegistrationArgs(TSharedRef<FToolElementRegistrationArgs> RegistrationArgs);

private:
	/** the FToolElementRegistrationArgs for the this tool element */
	TSharedRef<FToolElementRegistrationArgs> RegistrationArgs;
	
};

/** A Registry for FToolElements, where the FToolElements will generate the UI for a certain tool */
class FToolElementRegistry 
{
public:

	/**
	 *Gets the TSharedPtr<FToolElement> for the given FToolElementRegistrationKey& tool element key
	 *
	 *@param ToolElementKey the given FToolElementRegistrationKey& tool element key for which to find the tool element
	 */
	UE_API TSharedPtr<FToolElement> GetToolElementSP( FToolElementRegistrationKey& ToolElementKey);
	UE_API explicit FToolElementRegistry();

	static UE_API FToolElementRegistry& Get();

	/**
	 * Generates the widget for the tool element denoted by the FToolElementRegistrationKey, using the specified
	 * TSharedPtr<FToolElementRegistrationArgs> if provided (otherwise it will use the default registered registration
	 * arguments. If bUpdateRegistrationArgs is set to true and RegistrationArgsSP is not nullptr, the specified
	 * registration args will replace the existing ones for the tool element.
	 *
	 * @param ToolElementKeySR the TSharedRef<FToolElementRegistrationKey> which is the key to the tool element for 
	        which we want to generate the widget
	   @param RegistrationArgsSP The SharedPtr to the FToolElementRegistrationArgs for the tool element for which
			which we want to generate the widget
	 * @param bUpdateRegistrationArgs a bool that indicates whether we store any provided FToolElementRegistrationArgs
	 * as the new FToolElementRegistrationArgs for this tool element
	 */
	UE_API TSharedRef< SWidget > GenerateWidget(
		TSharedRef<FToolElementRegistrationKey> ToolElementKeySR,
		TSharedPtr<FToolElementRegistrationArgs> RegistrationArgsSP = nullptr,
		bool bUpdateRegistrationArgs = false);

	/** registers the FToolElement ToolElement
	 *
	 * @param ToolElement the FToolElement that is to be registered
	 */
	UE_API void RegisterElement(const TSharedRef<FToolElement> ToolElement);
	
	/** Unregisters the FToolElement ToolElement
	 *
	 * @param ToolElement the FToolElement that is to be registered
	 */
	UE_API void UnregisterElement(const TSharedRef<FToolElement> ToolElement);

private:
	/* the Map of tool element Key to tool element */
	TMap<FString, TSharedPtr<FToolElement>> ToolElementKeyToToolElementMap;
};

 

#undef UE_API
