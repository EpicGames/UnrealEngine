//  Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ToolElementRegistry.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#define UE_API WIDGETREGISTRATION_API

/** A class that provides the FToolbarRegistrationArgs for a UE toolbar  */
class FToolbarRegistrationArgs : public FToolElementRegistrationArgs
{
public:

	/** 
	 * the constructor for FToolbarRegistrationArgs
	 *
	 * @param InToolBarBuilder a TSharedRef to the FToolBarBuilder  which will be used by
	 * this to build up a toolbar SWidget
	 */
	UE_API FToolbarRegistrationArgs(TSharedRef<FToolBarBuilder> InToolBarBuilder);

	/**
	 * Generates the TSharedPtr<SWidget> for a toolbar
	 *
	 * @return the the TSharedPtr<SWidget> for a toolbar
	 */
	UE_API virtual TSharedPtr<SWidget> GenerateWidget() override;

	/** The FToolBarBuilder object used to build up a toolbar widget */
	TSharedRef<FToolBarBuilder> ToolBarBuilder;
	
};

#undef UE_API
