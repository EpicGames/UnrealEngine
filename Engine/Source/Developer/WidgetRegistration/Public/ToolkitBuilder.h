// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FToolkitWidgetStyle.h"
#include "IDetailsView.h"
#include "Framework/MultiBox/SToolBarButtonBlock.h"
#include "Framework/Commands/UICommandInfo.h"
#include "ToolElementRegistry.h"
#include "ToolkitBuilderConfig.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SSplitter.h"
#include "Layout/CategoryDrivenContentBuilderBase.h"

#define UE_API WIDGETREGISTRATION_API

class SWidget;
class FToolElement;

struct FToolkitSections
{
	TSharedPtr<STextBlock> ModeWarningArea = nullptr;
	TSharedPtr<STextBlock> ToolWarningArea = nullptr;
	TSharedPtr<SWidget> ToolPresetArea = nullptr;
	TSharedPtr<IDetailsView> DetailsView = nullptr;
	TSharedPtr<SWidget> ToolHeader = nullptr;
	TSharedPtr<SWidget> Header = nullptr;
	TSharedPtr<SWidget> Footer = nullptr;
};

/** A struct that provides the data for a single tool Palette*/
struct FToolPalette : TSharedFromThis<FToolPalette>
{
	FToolPalette(TSharedPtr<FUICommandInfo> InLoadToolPaletteAction,
		const TArray<TSharedPtr< FUICommandInfo >>& InPaletteActions) :
		LoadToolPaletteAction(InLoadToolPaletteAction)
	{
		for (const TSharedPtr< const FUICommandInfo > CommandInfo : InPaletteActions)
		{
			TSharedPtr<FButtonArgs> Button = MakeShareable(new FButtonArgs);
			Button->Command = CommandInfo;
			PaletteActions.Add(Button.ToSharedRef());
		}
	}

	/** The FUICommandInfo button which loads a particular set of toads */
	const TSharedPtr<FUICommandInfo> LoadToolPaletteAction;

	/** The ButtonArgs that has the data to initialize the buttons in the FToolPalette loaded by LoadToolPaletteAction */
	TArray<TSharedRef<FButtonArgs>> PaletteActions;

	/** The FUICommandList associated with this Palette */
	TSharedPtr<FUICommandList> PaletteActionsCommandList;
};

/** An FToolPalette to which you can add and remove actions */
struct FEditablePalette : public FToolPalette
{
	UE_API FEditablePalette(TSharedPtr<FUICommandInfo> InLoadToolPaletteAction,
		TSharedPtr<FUICommandInfo> InAddToPaletteAction,
		TSharedPtr<FUICommandInfo> InRemoveFromPaletteAction,
		FName InEditablePaletteName = FName(),
		FGetEditableToolPaletteConfigManager InGetConfigManager = FGetEditableToolPaletteConfigManager());

	/** The FUICommandInfo which adds an action to this palette */
	const TSharedPtr<FUICommandInfo> AddToPaletteAction;
	
	/** The FUICommandInfo which removes an action to this palette */
	const TSharedPtr<FUICommandInfo> RemoveFromPaletteAction;

	/** Delegate used by FToolkitBuilder that is called when an item is added/removed from the palette */
	FSimpleDelegate OnPaletteEdited;

	/**
	 * Given a reference to a FUICommandInfo, returns whether it is in the current Palette
	 *
	 * @param CommandName the name of the FUICommandInfo queried as to whether it is in the Palette
	 */
	UE_API bool IsInPalette(const FName CommandName) const;

	UE_API TArray<FString> GetPaletteCommandNames() const;

	UE_API void AddCommandToPalette(const FString CommandNameString);

	UE_API void RemoveCommandFromPalette(const FString CommandNameString);

protected:

	UE_API void SaveToConfig();

	UE_API void LoadFromConfig();
	
protected:
	/** The TArray of Command names that are the current FuiCommandInfo actions in this Palette */
	TArray<FString> PaletteCommandNameArray;

	/** The (unique) name attached to this palette, enables saving the palette contents into config if provided */
	FName EditablePaletteName;

	/** Delegate used to check if we have a config manager and get it */
	FGetEditableToolPaletteConfigManager GetConfigManager;
};

struct FToolkitBuilderArgs;

/*
 * A simple struct to carry initialization data for an FToolkitBuilder
 */
struct FToolkitBuilderArgs : public FCategoryDrivenContentBuilderArgs
{
	WIDGETREGISTRATION_API FToolkitBuilderArgs(const FName InToolbarCustomizationName);

	/** Name of the toolbar this mode uses and can be used by external systems to customize that mode toolbar */
	FName ToolbarCustomizationName;

	/** A TSharedPointer to the FUICommandList for the current mode */
	TSharedPtr<FUICommandList> ToolkitCommandList;

	/** The FToolkitSections which holds the sections defined for this Toolkit */
	TSharedPtr<FToolkitSections> ToolkitSections;

	/** If SelectedCategoryTitleVisibility == EVisibility::Visible, the selected
	 * category title is visible, else it is not displayed  */
	EVisibility SelectedCategoryTitleVisibility;
};

class FToolkitBuilder : public FCategoryDrivenContentBuilderBase
{
public:
	/**
	 * Constructor
	 * 
	 * @param ToolbarCustomizationName the name of the  customization fr the category toolbar 
	 * @param InToolkitCommandList  the toolkit FUICommandList
	 * @param InToolkitSections The FToolkitSections for this toolkit builder
	 */
	UE_API explicit FToolkitBuilder(
	FName ToolbarCustomizationName,
	TSharedPtr<FUICommandList> InToolkitCommandList,
	TSharedPtr<FToolkitSections> InToolkitSections);

	/** default constructor */
	UE_API explicit FToolkitBuilder(FToolkitBuilderArgs& Args);

	UE_API virtual ~FToolkitBuilder() override;

	UE_API void SetActivePaletteCommandsVisibility(EVisibility Visibility);

	/**
	 * @return the Visibility state of the active palette command buttons
	 */
	UE_API EVisibility GetActivePaletteCommandsVisibility() const;

	/**
	 * Adds the FToolPalette Palette to this FToolkitBuilder
	 *
	 * @param Palette the FToolPalette being added to this FToolkitBuilder
	 */
	UE_API void AddPalette(TSharedPtr<FToolPalette> Palette);

	/**
	 * Adds the FEditablePalette Palette to this FToolkitBuilder
	 *
	 * @param Palette the FEditablePalette being added to this FToolkitBuilder
	 */
	UE_API void AddPalette(TSharedPtr<FEditablePalette> Palette);

	/** Cleans up any previously set data in this FToolkitBuilder and reset the members to their initial values  */
	UE_API virtual void ResetWidget() override final;

	/** Updates the Toolkit. This should be called after any changes to the data of this FToolkitBuilder, and the UI
	 * will be regenerated to reflect it.  */
	UE_API virtual void UpdateWidget() override;

	/** @return true is there is an active palette selected, else it returns false */
	UE_API bool HasActivePalette() const;

	/*
	 * Loads the palette for the FUICommandInfo Command on first visit to the mode.
	 *
	 *  @param Command the FUICommandInfo which defines the palette that will be loaded on first visit to the mode.
	 */
	UE_API void SetActivePaletteOnLoad(const FUICommandInfo* Command);
	
	/**
	 * @return true if the FUICommandInfo with the name CommandName is the active tool palette,
	 * else it returns false
	 *
	 * @param CommandName the name of the FUICommandInfo we are checking to see if it is the active tool palette
	 */
	UE_API ECheckBoxState IsActiveToolPalette(FName CommandName) const;
	
	/*
	 * Initializes the data necessary to build the category toolbar
	 */
	UE_API void InitializeCategoryToolbar() override;

	/**
	 * Initializes the data necessary to build the category toolbar
	 *
	 * @param bInitLoadToolPaletteMap If true, the map which holds the load
	 * action names as keys and the ToolPalettes as value will be cleared and available
	 * to add new data when the method completes. Otherwise, the map will keep
	 * the load action name to ToolPalette mappings that it was initially set up with. 
	 */
	UE_API void InitializeCategoryToolbar(bool bInitLoadToolPaletteMap);

	/**
	 * Sets the visibility of the category toolbar. Call RefreshCategoryToolbarWidget to apply
	 * changes in visibility.
	 * @param Visibility the visibility state to apply
	 */
	UE_API void SetCategoryToolbarVisibility(EVisibility Visibility);

	/**
	 * Sets the display name for the active tool
	 *
	 * @param InActiveToolDisplayName an FText that holds the name of the currently active tool
	 */
	UE_API void SetActiveToolDisplayName(FText InActiveToolDisplayName);

	/**
	 * @returns the display name for the active tool
	 */
	UE_API FText GetActiveToolDisplayName() const;

	/**
	 * Fills the OutCommands TArray with TSharedPtr<const FUICommandInfo> instances that are present in the FEditablePalette
	 * EditablePalette
	 *
	 * @param EditablePalette the FEditablePalette whose FUICommandInfo
	 */
	UE_API void GetCommandsForEditablePalette(TSharedRef<FEditablePalette> EditablePalette, TArray<TSharedPtr<const FUICommandInfo>>& OutCommands);

	/**
	 * @returns the name of the Active Palette if one is available, else it returns NAME_NONE
	 */
	UE_API FName GetActivePaletteName() const;

	/** Name of the toolbar this mode uses and can be used by external systems to customize that mode toolbar */
	FName ToolbarCustomizationName;

	/** The map of the command name to the ButtonArgs for it  */
	TMap<FString, TSharedPtr<FButtonArgs>> PaletteCommandNameToButtonArgsMap;

	/** The map of the command name to the ButtonArgs for it  */
	TMap<FString, TSharedPtr<FToolPalette>> LoadCommandNameToToolPaletteMap;

	/** Map of command name to the actual command for all commands belonging to this palette */
	TMap<FString, TSharedPtr<const FUICommandInfo>> PaletteCommandInfos;
	
	/** A TSharedPointer to the FUICommandList for the current mode */
	TSharedPtr<FUICommandList> ToolkitCommandList;

	/** A TArray of FEditablePalettes, kept to update the commands which are on them */
	TArray<TSharedRef<FEditablePalette>> EditablePalettesArray;

	/** The tool palette which is currently loaded/active */
	TSharedPtr<FToolPalette> ActivePalette;

	/** The SVerticalBox which holds the tool palette (the Buttons which load each tool) */
	TSharedPtr<SVerticalBox> ToolPaletteWidget;
	
	/** the FName of each Load palette command to the FToolbarBuilder for the palette which it loads */
	TMap<FName, TSharedPtr<FToolBarBuilder>> LoadCommandNameToPaletteToolbarBuilderMap;

	/** Updates the Editable Palette with any commands that are on it */
	UE_API void UpdateEditablePalette(TSharedRef<FEditablePalette> EditablePalette);

	UE_API void OnEditablePaletteEdited(TSharedRef<FEditablePalette> EditablePalette);

	/** Resets the tool palette widget, on which the buttons for the currently chosen toolset are shown */
	UE_API void ResetToolPaletteWidget();

	/** The array of Tool Palettes on display. This array is used to keep a handle
	* on all of the created FToolElements so that we can unregister them upon destruction */
	TArray<TSharedRef<FToolElement>> ToolPaletteElementArray;

private:
	/**
	 * Creates the UI for the Palette specified by the FToolPalette Palette
	 *
	 * @param Palette the FToolPalette for which this method will build the UI 
	 */
	UE_API void CreatePalette(TSharedPtr<FToolPalette> Palette);

	/**
	 * Creates the UI for the Palette specified by the FToolPalette Palette if it is not currently active,
	 * else it removes it.
	 *
	 * @param Palette the FToolPalette for which this method will build the UI 
	 */
	UE_API void TogglePalette(TSharedPtr<FToolPalette> Palette);
	
	/**
	 * Retrieves the context menu content for the FUICommandInfo with the command name CommandName
	 *
	 * @param CommandName the command name of the FUICommandInfo to get the context menu widget for
	 * @return the TSharedRef<SWidget> which contains the context menu for the FUICommandInfo with the name CommandName 
	 */
	UE_API TSharedRef<SWidget> GetContextMenuContent(const FName CommandName);
	
	UE_API void CreatePaletteWidget(FToolPalette& Palette, FToolElement& Element);
	
	/**
	 * Adds the FUICommandInfo with the command name CommandNameString to the FEditablePalette Palette if
	 * it is not on Palette, and removes the FUICommandInfo from Palette if it is on the Palette
	 *
	 * @param Palette the FEditablePalette to which the FUICommandInfo will be toggled on/off
	 * @param CommandNameString the name of the FUICommandInfo command which will be toggled on/off of Palette
	 */
	UE_API void ToggleCommandInPalette(TSharedRef<FEditablePalette> Palette, FString CommandNameString);
		
	/* @return true if the Toolkit builder has some tools that are currently active/selected */
	UE_API bool HasSelectedToolSet() const;

	/**
	 * Given the active category name, update the toolkit content
	 * 
	 * @param ActiveCategoryName 
	 */
	UE_API virtual void UpdateContentForCategory( FName ActiveCategoryName = NAME_None, FText InActiveCategoryText = FText::GetEmpty() ) override;

	/**
	 * Gets the EVisibility of the active tool title. This should only be visible if a tool is currently chosen in the palette.
	 *
	 * @return the EVisibility of the active tool title
	 */
	UE_API EVisibility GetActiveToolTitleVisibility() const;
	
	/** If SelectedCategoryTitleVisibility == EVisibility::Visible, the selected
    * category title is visible, else it is not displayed. By default the selected category title is Collapsed  */	
	EVisibility SelectedCategoryTitleVisibility = EVisibility::Collapsed;
	
	/** The display name of the currently active tool */
	FText ActiveToolDisplayName = FText::GetEmpty();

	/** The array of Command names, in the correct display order */
	TArray<FName> LoadCommandArray;

protected:
	/** @return a TSharedPointer to the FToolbarBuilder with the FUICommandInfos that load the Palettes */
	UE_API TSharedRef<SWidget> GetToolPaletteWidget() const;
	
	/** If ActivePaletteButtonVisibility == EVisibility::Visible, the command buttons in the active palette
	 * are visible, otherwise they are not displayed. By default the state is Visible  */
	EVisibility ActivePaletteButtonVisibility = EVisibility::Visible;

	/** The FToolkitSections which holds the sections defined for this Toolkit */
	TSharedPtr<FToolkitSections> ToolkitSections;
};

#undef UE_API
