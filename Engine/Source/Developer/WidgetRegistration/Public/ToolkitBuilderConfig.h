// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"

#include "ToolkitBuilderConfig.generated.h"

// The actual struct that contains the properties saved for an FEditableToolPalette
USTRUCT()
struct FEditableToolPaletteSettings
{
	GENERATED_BODY()
	
public:
	
	UPROPERTY()
	TArray<FString> PaletteCommandNames;
};

// Interface class for FEditableToolPalette to interact with EditorConfig which is an Editor only module currently
class IEditableToolPaletteConfigManager
{
public:

	virtual FEditableToolPaletteSettings* GetMutablePaletteConfig(const FName& InstanceName) = 0;
	virtual const FEditableToolPaletteSettings* GetConstPaletteConfig(const FName& InstanceName) = 0;
	virtual void SavePaletteConfig(const FName& InstanceName) = 0;
	virtual ~IEditableToolPaletteConfigManager() = default;
};

// Delegate passed into FEditableToolPalette that is used to get the actual config manager instance
DECLARE_DELEGATE_RetVal(IEditableToolPaletteConfigManager*, FGetEditableToolPaletteConfigManager)

