// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"
#include "Templates/SharedPointer.h"

#define UE_API WIDGETREGISTRATION_API

class ISlateStyle;

/** FToolkitStyle is the FSlateStyleSet that defines styles for FToolkitBuilders  */
class FToolkitStyle
	: public FSlateStyleSet
{
public:
	
	static UE_API FName StyleName;
	UE_API FToolkitStyle();
	static UE_API void Initialize();
	static UE_API void Shutdown();
	static UE_API const ISlateStyle& Get();
	UE_API virtual const FName& GetStyleSetName() const override;

private:

	static UE_API TSharedRef< class FSlateStyleSet > Create();
	static UE_API TSharedPtr< class FSlateStyleSet > StyleSet;
};

#undef UE_API
