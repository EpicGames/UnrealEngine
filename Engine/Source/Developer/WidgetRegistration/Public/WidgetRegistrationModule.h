// Copyright Epic Games, Inc. All Rights Reserved.

// #include "ToolkitStyle.h"

#pragma once

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

// @TODO: ~enable ToolkitStyle set 

class FWidgetRegistrationModule
	: public IModuleInterface
{
public:

	// IModuleInterface interface
	
	virtual void StartupModule() override;
	
	virtual void ShutdownModule() override;

};