// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class WidgetRegistration : ModuleRules
{
	public WidgetRegistration(ReadOnlyTargetRules Target) : base(Target)
	{		
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"SlateCore",
				"Core",
				"Slate",
				"Engine",
				"ToolMenus",
				"CoreUObject",
				"EditorConfig"
			});

		PublicIncludePathModuleNames.AddRange(
			new string[] {
				"ToolMenus",
			});
	}
}